{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "Mesh Resizing",
    "Description" : "Mesh Resizing",
    "Category": "Editor",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "https://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "EnabledByDefault" : false,
    "CanContainContent" : true,
	"IsExperimentalVersion": true,
    "Installed" : false,
	"Modules": [
		{
			"Name": "MeshResizingCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshResizingDataflowNodes",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshResizingEditorTools",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "GeometryCollectionPlugin",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "HairStrands",
			"Enabled": true
		}
	]
}
