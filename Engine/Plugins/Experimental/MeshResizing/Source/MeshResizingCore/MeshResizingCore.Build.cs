// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MeshResizingCore : ModuleRules
	{
        public MeshResizingCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Chaos",
					"Core",
					"CoreUObject",
					"GeometryCore",
					"HairStrandsCore",
					"MeshDescription"
				}
			);
		}
	}
}
