// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/BaseBodyTools.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexAttribute.h"
#include "Async/TaskGraphInterfaces.h"
#include "Tasks/Task.h"

namespace UE::MeshResizing
{
	namespace Private
	{
		class FVertexMappingAttribute final :
			public UE::Geometry::TDynamicVertexAttribute<int32,1,UE::Geometry::FDynamicMesh3>
		{
			typedef UE::Geometry::TDynamicVertexAttribute<int32, 1, UE::Geometry::FDynamicMesh3> Base;
		public:
			FVertexMappingAttribute() = default;

			FVertexMappingAttribute(UE::Geometry::FDynamicMesh3* InParent) 
				: Base(InParent, false)
			{
			}

			virtual ~FVertexMappingAttribute() override = default;

			virtual int32 GetDefaultAttributeValue() override
			{
				return INDEX_NONE;
			}

			void InitializeFromArray(TConstArrayView<int32> VertexIDMap)
			{
				check(VertexIDMap.Num() == Parent->MaxVertexID());
				AttribValues.Resize(Parent->MaxVertexID());

				for (int32 VertexID : Parent->VertexIndicesItr())
				{
					SetMappedValue(VertexID, VertexIDMap[VertexID]);
				}
			}

			void SetMappedValue(int32 VertexID, int32 Value)
			{
				AttribValues[VertexID] = Value;
			}

			int32 GetMappedValue(int32 VertexID) const
			{
				return AttribValues[VertexID];
			}

		private:
			virtual void SetAttributeFromLerp(int SetAttribute, int AttributeA, int AttributeB, double Alpha) override
			{
				// Do not interpolate
				AttribValues[SetAttribute] = INDEX_NONE;
			}

			virtual void SetAttributeFromBary(int SetAttribute, int AttributeA, int AttributeB, int AttributeC, const FVector3d& BaryCoords) override
			{
				// Do not interpolate
				AttribValues[SetAttribute] = INDEX_NONE;
			}
		};

		static const FVertexMappingAttribute* GetVertexMappingAttribute(const UE::Geometry::FDynamicMesh3& Mesh, const FName& AttrName)
		{
			using namespace UE::Geometry;
			const FDynamicMeshAttributeSet* const Attributes = Mesh.Attributes();
			if (Attributes && Attributes->HasAttachedAttribute(AttrName))
			{
				return static_cast<const FVertexMappingAttribute*>(Attributes->GetAttachedAttribute(AttrName));
			}
			return nullptr;
		}

		static void TransferNormalOverlayValues(const UE::Geometry::FDynamicMeshNormalOverlay& TargetOverlay, UE::Geometry::FDynamicMeshNormalOverlay& ProxyOverlay, const FVertexMappingAttribute& SourceAttr, const TMap<int32, int32>& MappedTargetToMeshTarget)
		{
			const int32 NumElements = ProxyOverlay.MaxElementID();
			const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), NumElements), 1);
			constexpr int32 MinElementsByTask = 20;
			const int32 ElementsByTask = FMath::Max(FMath::DivideAndRoundUp(NumElements, NumTasks), MinElementsByTask);
			const int32 NumBatches = FMath::DivideAndRoundUp(NumElements, ElementsByTask);
			TArray<UE::Tasks::FTask> PendingTasks;
			PendingTasks.Reserve(NumBatches);
			for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
			{
				const int32 StartIndex = BatchIndex * ElementsByTask;
				int32 EndIndex = (BatchIndex + 1) * ElementsByTask;
				EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(NumElements, EndIndex) : EndIndex;
				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&ProxyOverlay, &SourceAttr, &TargetOverlay, &MappedTargetToMeshTarget, StartIndex, EndIndex]()
				{
					TArray<int32> TargetVertexElements; // Reused array
					for (int32 ProxyElementID = StartIndex; ProxyElementID < EndIndex; ProxyElementID++)
					{
						if (ProxyOverlay.IsElement(ProxyElementID))
						{
							const int32 ProxyVID = ProxyOverlay.GetParentVertex(ProxyElementID);
							const int32 MappedVID = SourceAttr.GetMappedValue(ProxyVID);
							if (const int32* const TargetVID = MappedTargetToMeshTarget.Find(MappedVID))
							{
								TargetOverlay.GetVertexElements(*TargetVID, TargetVertexElements);
								if (ensure(TargetVertexElements.Num() > 0))
								{
									ensure(TargetVertexElements.Num() == 1);
									ProxyOverlay.SetElement(ProxyElementID, TargetOverlay.GetElement(TargetVertexElements[0]));
								}
							}
						}
					}
				});
				PendingTasks.Add(PendingTask);
			}
			UE::Tasks::Wait(PendingTasks);
		}
		static void InterpolateNormalOverlayValues(const UE::Geometry::FDynamicMeshNormalOverlay& SourceOverlay, const UE::Geometry::FDynamicMeshNormalOverlay& TargetOverlay, float BlendAlpha, UE::Geometry::FDynamicMeshNormalOverlay& ProxyOverlay)
		{
			check(SourceOverlay.ElementCount() == TargetOverlay.ElementCount());
			check(SourceOverlay.ElementCount() == ProxyOverlay.ElementCount());
			const float OneMinusAlpha = 1.f - BlendAlpha;
			for (int32 ProxyElementID : ProxyOverlay.ElementIndicesItr())
			{
				ProxyOverlay.SetElement(ProxyElementID, OneMinusAlpha * SourceOverlay.GetElement(ProxyElementID) + BlendAlpha * TargetOverlay.GetElement(ProxyElementID));
			}
		}
	}

	const FName FBaseBodyTools::ImportedVertexVIDsAttrName = FName(TEXT("ImportedVertexVIDsAttr"));
	const FName FBaseBodyTools::RawPointIndicesVIDsAttrName = FName(TEXT("RawPointIndicesVIDsAttr"));

	bool FBaseBodyTools::AttachVertexMappingData(const FName& AttrName, const TArray<int32>& Data, UE::Geometry::FDynamicMesh3& Mesh)
	{
		// Copied from FNonManifoldMappingSupport
		using namespace UE::Geometry;
		const int32 MaxVID = Mesh.MaxVertexID();
		FDynamicMeshAttributeSet* const  Attributes = Mesh.Attributes();
		if (Data.Num() < MaxVID || !Attributes)
		{
			return false;
		}

		// Replace existing buffer.
		if (Attributes->HasAttachedAttribute(AttrName))
		{
			Attributes->RemoveAttribute(AttrName);
		}
		Private::FVertexMappingAttribute* SrcMeshVIDAttr = new Private::FVertexMappingAttribute(&Mesh);
		SrcMeshVIDAttr->SetName(AttrName);
		Mesh.Attributes()->AttachAttribute(AttrName, SrcMeshVIDAttr);
		SrcMeshVIDAttr->InitializeFromArray(Data);
		return true;
	}

	bool FBaseBodyTools::GenerateResizableProxyFromVertexMappingData(const UE::Geometry::FDynamicMesh3& SourceMesh, const FName& SourceMappingName, const UE::Geometry::FDynamicMesh3& TargetMesh, const FName& TargetMappingName, UE::Geometry::FDynamicMesh3& ProxyMesh)
	{
		using namespace UE::Geometry;
		const Private::FVertexMappingAttribute* const SourceAttr = Private::GetVertexMappingAttribute(SourceMesh, SourceMappingName);
		const Private::FVertexMappingAttribute* const TargetAttr = Private::GetVertexMappingAttribute(TargetMesh, TargetMappingName);
		if (!SourceAttr || !TargetAttr)
		{
			return false;
		}

		TMap<int32, int32> MappedTargetToMeshTarget;

		for (int32 VertexID : TargetMesh.VertexIndicesItr())
		{
			const int32 TargetValue = TargetAttr->GetMappedValue(VertexID);
			// Multiple mesh vertices may have the same mapped value. Unless the mesh has been modified, they should all have the same vertex and wedge data since these came from SKMs
			// Just track last we iterate to.
			MappedTargetToMeshTarget.Add(TargetValue, VertexID); 
		}

		// Start with source mesh topology
		ProxyMesh = SourceMesh;

		// Update vertex data to match target mesh values.
		for (int32 SourceVertexID : SourceMesh.VertexIndicesItr())
		{
			const int32 MappedValue = SourceAttr->GetMappedValue(SourceVertexID);

			if (const int32* const TargetVertexID = MappedTargetToMeshTarget.Find(MappedValue))
			{
				ProxyMesh.SetVertex(SourceVertexID, TargetMesh.GetVertexRef(*TargetVertexID));
			}
		}

		// Currently assuming materials, uvs, colors match between source and target.
		
		const FDynamicMeshNormalOverlay* const TargetNormalOverlay = TargetMesh.Attributes()->PrimaryNormals();
		const FDynamicMeshNormalOverlay* const TargetTangentOverlay = TargetMesh.Attributes()->PrimaryTangents();
		const FDynamicMeshNormalOverlay* const TargetBiTangentOverlay = TargetMesh.Attributes()->PrimaryBiTangents();

		FDynamicMeshNormalOverlay* const ProxyNormalOverlay = ProxyMesh.Attributes()->PrimaryNormals();
		FDynamicMeshNormalOverlay* const ProxyTangentOverlay = ProxyMesh.Attributes()->PrimaryTangents();
		FDynamicMeshNormalOverlay* const ProxyBiTangentOverlay = ProxyMesh.Attributes()->PrimaryBiTangents();

		if (TargetNormalOverlay && ProxyNormalOverlay)
		{
			Private::TransferNormalOverlayValues(*TargetNormalOverlay, *ProxyNormalOverlay, *SourceAttr, MappedTargetToMeshTarget);
		}
		if (TargetTangentOverlay && ProxyTangentOverlay)
		{
			Private::TransferNormalOverlayValues(*TargetTangentOverlay, *ProxyTangentOverlay, *SourceAttr, MappedTargetToMeshTarget);
		}
		if (TargetBiTangentOverlay && ProxyBiTangentOverlay)
		{
			Private::TransferNormalOverlayValues(*TargetBiTangentOverlay, *ProxyBiTangentOverlay, *SourceAttr, MappedTargetToMeshTarget);
		}

		return true;
	}

	bool FBaseBodyTools::InterpolateResizableProxy(const UE::Geometry::FDynamicMesh3& SourceMesh, const UE::Geometry::FDynamicMesh3& TargetMesh, float BlendAlpha, UE::Geometry::FDynamicMesh3& ProxyMesh)
	{
		using namespace UE::Geometry;
		if (SourceMesh.VertexCount() != TargetMesh.VertexCount())
		{
			return false;
		}

		// in case there are attributes we're not interpolating, choose the closest for those?
		ProxyMesh = BlendAlpha < 0.5f ? SourceMesh : TargetMesh;

		const float OneMinusAlpha = 1.f - BlendAlpha;
		for (int32 VertexID : ProxyMesh.VertexIndicesItr())
		{
			ProxyMesh.SetVertex(VertexID, OneMinusAlpha * SourceMesh.GetVertexRef(VertexID) + BlendAlpha * TargetMesh.GetVertexRef(VertexID));
		}
		// Currently assuming materials, uvs, colors match between source and target.

		const FDynamicMeshNormalOverlay* const SourceNormalOverlay = SourceMesh.Attributes()->PrimaryNormals();
		const FDynamicMeshNormalOverlay* const SourceTangentOverlay = SourceMesh.Attributes()->PrimaryTangents();
		const FDynamicMeshNormalOverlay* const SourceBiTangentOverlay = SourceMesh.Attributes()->PrimaryBiTangents();

		const FDynamicMeshNormalOverlay* const TargetNormalOverlay = TargetMesh.Attributes()->PrimaryNormals();
		const FDynamicMeshNormalOverlay* const TargetTangentOverlay = TargetMesh.Attributes()->PrimaryTangents();
		const FDynamicMeshNormalOverlay* const TargetBiTangentOverlay = TargetMesh.Attributes()->PrimaryBiTangents();

		FDynamicMeshNormalOverlay* const ProxyNormalOverlay = ProxyMesh.Attributes()->PrimaryNormals();
		FDynamicMeshNormalOverlay* const ProxyTangentOverlay = ProxyMesh.Attributes()->PrimaryTangents();
		FDynamicMeshNormalOverlay* const ProxyBiTangentOverlay = ProxyMesh.Attributes()->PrimaryBiTangents();

		if (SourceNormalOverlay && TargetNormalOverlay && SourceNormalOverlay->ElementCount() == TargetNormalOverlay->ElementCount())
		{
			check(ProxyNormalOverlay);
			Private::InterpolateNormalOverlayValues(*SourceNormalOverlay, *TargetNormalOverlay, BlendAlpha, *ProxyNormalOverlay);
		}
		if (SourceTangentOverlay && TargetTangentOverlay && SourceTangentOverlay->ElementCount() == TargetTangentOverlay->ElementCount())
		{
			check(ProxyTangentOverlay);
			Private::InterpolateNormalOverlayValues(*SourceTangentOverlay, *TargetTangentOverlay, BlendAlpha, *ProxyTangentOverlay);
		}
		if (SourceBiTangentOverlay && TargetBiTangentOverlay && SourceBiTangentOverlay->ElementCount() == TargetBiTangentOverlay->ElementCount())
		{
			check(ProxyBiTangentOverlay);
			Private::InterpolateNormalOverlayValues(*SourceBiTangentOverlay, *TargetBiTangentOverlay, BlendAlpha, *ProxyBiTangentOverlay);
		}
		return true;
	}
}