// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/CustomRegionResizing.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "IndexTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomRegionResizing)

namespace UE::MeshResizing
{
	namespace Private
	{
		static bool CalculateTriangleFrameNoAttributes(const UE::Geometry::FTriangle3d& Triangle, FVector3f& OutTangentU, FVector3f& OutTangentV, FVector3f& OutNormal)
		{
			OutTangentU = FVector3f(Triangle.V[1] - Triangle.V[0]).GetSafeNormal();
			OutNormal = OutTangentU.Cross(FVector3f(Triangle.V[2] - Triangle.V[0])).GetSafeNormal();
			OutTangentV = OutNormal.Cross(OutTangentU);
			return !OutTangentU.IsZero() && !OutTangentV.IsZero() && !OutNormal.IsZero();
		}

		static bool CalculateFrameFromAttributes(const UE::Geometry::FDynamicMesh3& SourceMesh, const int32 FaceIndex, const FVector3f& Barys, FVector3f& OutTangentU, FVector3f& OutTangentV, FVector3f& OutNormal)
		{
			check(SourceMesh.HasAttributes() && SourceMesh.Attributes()->NumNormalLayers() >= 2);

			const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = SourceMesh.Attributes()->PrimaryNormals();
			check(NormalOverlay);
			NormalOverlay->GetTriBaryInterpolate(FaceIndex, &Barys.X, &OutNormal.X);
			OutNormal = OutNormal.GetSafeNormal();

			const UE::Geometry::FDynamicMeshNormalOverlay* const TangentOverlay = SourceMesh.Attributes()->PrimaryTangents();
			check(TangentOverlay);
			TangentOverlay->GetTriBaryInterpolate(FaceIndex, &Barys.X, &OutTangentU.X);
			OutTangentU = OutTangentU.GetSafeNormal();
			OutTangentU = (OutTangentU - (OutTangentU | OutNormal) * OutNormal).GetSafeNormal();
			OutTangentV = OutNormal.Cross(OutTangentU);
			return !OutTangentU.IsZero() && !OutTangentV.IsZero() && !OutNormal.IsZero();
		}
	}

	void FCustomRegionResizing::GenerateCustomRegion(const TConstArrayView<FVector3f>& BoundPositions, const UE::Geometry::FDynamicMesh3& SourceMesh, const TSet<int32>& RegionVertices, FMeshResizingCustomRegion& OutData)
	{
		
		OutData.Reset();
		OutData.RegionVertices.Reserve(RegionVertices.Num());

		FVector3d Centroid(0.);
		for (int32 RigidBoundVertex : RegionVertices)
		{
			if (BoundPositions.IsValidIndex(RigidBoundVertex))
			{
				OutData.RegionVertices.Add(RigidBoundVertex);
				Centroid += FVector3d(BoundPositions[RigidBoundVertex]);
			}
		}
		if (OutData.RegionVertices.Num() == 0)
		{
			return;
		}

		Centroid /= OutData.RegionVertices.Num();

		const bool bHasValidNormalAndTangentLayers = SourceMesh.HasAttributes() && SourceMesh.Attributes()->NumNormalLayers() >= 2;

		UE::Geometry::IMeshSpatial::FQueryOptions QueryOptions;

		if (bHasValidNormalAndTangentLayers)
		{
			QueryOptions.TriangleFilterF = [&SourceMesh, &Centroid](int32 FaceIndex)
				{
					UE::Geometry::FDistPoint3Triangle3d DistQuery = UE::Geometry::TMeshQueries<UE::Geometry::FDynamicMesh3>::TriangleDistance(SourceMesh, FaceIndex, Centroid);

					FVector3f TangentU, TangentV, Normal;
					return Private::CalculateFrameFromAttributes(SourceMesh, FaceIndex, FVector3f(DistQuery.TriangleBaryCoords), TangentU, TangentV, Normal);
				};
		}
		else
		{
			QueryOptions.TriangleFilterF = [&SourceMesh](int32 FaceIndex)
				{
					UE::Geometry::FTriangle3d Tri;
					SourceMesh.GetTriVertices(FaceIndex, Tri.V[0], Tri.V[1], Tri.V[2]);

					FVector3f TangentU, TangentV, Normal;
					return Private::CalculateTriangleFrameNoAttributes(Tri, TangentU, TangentV, Normal);
				};
		}

		UE::Geometry::FDynamicMeshAABBTree3 AABBTree(&SourceMesh);
		double DistSq;
		OutData.SourceFaceIndex = AABBTree.FindNearestTriangle(Centroid, DistSq, QueryOptions);
		static_assert(IndexConstants::InvalidID == INDEX_NONE);
		if (OutData.SourceFaceIndex == INDEX_NONE)
		{
			return;
		}

		UE::Geometry::FDistPoint3Triangle3d DistQuery = UE::Geometry::TMeshQueries<UE::Geometry::FDynamicMesh3>::TriangleDistance(SourceMesh, OutData.SourceFaceIndex, Centroid);
		const float Distance = FMath::Sqrt(DistQuery.ComputeResult());
		OutData.SourceBaryCoords = FVector3f(DistQuery.TriangleBaryCoords);
		OutData.SourceOrigin = DistQuery.ClosestTrianglePoint;

		// Calculate local frame for triangle.
		if (bHasValidNormalAndTangentLayers)
		{
			if (!ensure(Private::CalculateFrameFromAttributes(SourceMesh, OutData.SourceFaceIndex, OutData.SourceBaryCoords, OutData.SourceAxis0, OutData.SourceAxis1, OutData.SourceAxis2)))
			{
				OutData.Reset();
				return;
			}
		}
		else
		{
			if (!ensure(Private::CalculateTriangleFrameNoAttributes(DistQuery.Triangle, OutData.SourceAxis0, OutData.SourceAxis1, OutData.SourceAxis2)))
			{
				OutData.Reset();
				return;
			}
		}
		const FMatrix TriangleMatrix(FVector3d(OutData.SourceAxis0), FVector3d(OutData.SourceAxis1), FVector3d(OutData.SourceAxis2), OutData.SourceOrigin);
		OutData.RegionBoundsCentroid = FVector3f(TriangleMatrix.InverseTransformPosition(Centroid));

		// Need to calculate transformed extents.
		OutData.RegionBoundsExtents = FVector3f(0.f);
		for (int32 RigidBoundVertex : OutData.RegionVertices)
		{
			const FVector3f TransformedPoint = FVector3f(TriangleMatrix.InverseTransformPosition(FVector3d(BoundPositions[RigidBoundVertex])));
			const FVector3f TransformedDelta = (TransformedPoint - OutData.RegionBoundsCentroid).GetAbs();
			OutData.RegionBoundsExtents = OutData.RegionBoundsExtents.ComponentMax(TransformedDelta);
		}
		const FVector3f MinBounds = OutData.RegionBoundsCentroid - OutData.RegionBoundsExtents;
		FVector3f RecipBoundsSize;
		RecipBoundsSize.X = OutData.RegionBoundsExtents.X > UE_SMALL_NUMBER ? .5f / OutData.RegionBoundsExtents.X : 0.f;
		RecipBoundsSize.Y = OutData.RegionBoundsExtents.Y > UE_SMALL_NUMBER ? .5f / OutData.RegionBoundsExtents.Y : 0.f;
		RecipBoundsSize.Z = OutData.RegionBoundsExtents.Z > UE_SMALL_NUMBER ? .5f / OutData.RegionBoundsExtents.Z : 0.f;

		OutData.RegionVertexCoords.Reset(OutData.RegionVertices.Num());
		for (int32 RigidBoundVertex : OutData.RegionVertices)
		{
			// Transform to bounds space
			const FVector3f TransformedPoint = FVector3f(TriangleMatrix.InverseTransformPosition(FVector3d(BoundPositions[RigidBoundVertex])));
			OutData.RegionVertexCoords.Emplace((TransformedPoint - MinBounds) * RecipBoundsSize);
		}
	}

	bool FCustomRegionResizing::CalculateFrameForCustomRegion(const UE::Geometry::FDynamicMesh3& SourceMesh, const FMeshResizingCustomRegion& BindingGroup, FVector3d& OutOrigin, FVector3f& OutTangentU, FVector3f& OutTangentV, FVector3f& OutNormal)
	{

		UE::Geometry::FTriangle3d Triangle;
		SourceMesh.GetTriVertices(BindingGroup.SourceFaceIndex, Triangle.V[0], Triangle.V[1], Triangle.V[2]);

		OutOrigin = Triangle.BarycentricPoint(FVector(BindingGroup.SourceBaryCoords));

		const bool bHasValidNormalAndTangentLayers = SourceMesh.HasAttributes() && SourceMesh.Attributes()->NumNormalLayers() >= 2;
		if (bHasValidNormalAndTangentLayers)
		{
			return Private::CalculateFrameFromAttributes(SourceMesh, BindingGroup.SourceFaceIndex, BindingGroup.SourceBaryCoords, OutTangentU, OutTangentV, OutNormal);
		}
		else
		{
			return Private::CalculateTriangleFrameNoAttributes(Triangle, OutTangentU, OutTangentV, OutNormal);
		}
	}

	void FCustomRegionResizing::InterpolateCustomRegionPoints(const FMeshResizingCustomRegion& BindingGroup, const TConstArrayView<FVector3d>& BoundsCorners, TArrayView<FVector3f> BoundPositions)
	{
		check(BoundsCorners.Num() == 8);
		for (int32 Index = 0; Index < BindingGroup.RegionVertices.Num(); ++Index)
		{
			if (BoundPositions.IsValidIndex(BindingGroup.RegionVertices[Index]))
			{
				// BoundsCorners are in the order of FOrientedBox::CalcVertices
				// Xijk = Center + (Signs[i] * ExtentX) * AxisX + (Signs[j] * ExtentY) * AxisY + (Signs[k] * ExtentZ) * AxisZ where Signs[] = { -1, 1 };
				// BoundsCorners[0] = X000
				// BoundsCorners[1] = X001
				// BoundsCorners[2] = X010
				// BoundsCorners[3] = X011
				// BoundsCorners[4] = X100
				// BoundsCorners[5] = X101
				// BoundsCorners[6] = X110
				// BoundsCorners[7] = X111
				const FVector3d X00 = BoundsCorners[0] * (1. - BindingGroup.RegionVertexCoords[Index].X) + BoundsCorners[4] * BindingGroup.RegionVertexCoords[Index].X;
				const FVector3d X01 = BoundsCorners[1] * (1. - BindingGroup.RegionVertexCoords[Index].X) + BoundsCorners[5] * BindingGroup.RegionVertexCoords[Index].X;
				const FVector3d X10 = BoundsCorners[2] * (1. - BindingGroup.RegionVertexCoords[Index].X) + BoundsCorners[6] * BindingGroup.RegionVertexCoords[Index].X;
				const FVector3d X11 = BoundsCorners[3] * (1. - BindingGroup.RegionVertexCoords[Index].X) + BoundsCorners[7] * BindingGroup.RegionVertexCoords[Index].X;

				const FVector X0 = X00 * (1. - BindingGroup.RegionVertexCoords[Index].Y) + X10 * BindingGroup.RegionVertexCoords[Index].Y;
				const FVector X1 = X01 * (1. - BindingGroup.RegionVertexCoords[Index].Y) + X11 * BindingGroup.RegionVertexCoords[Index].Y;

				const FVector X = X0 * (1. - BindingGroup.RegionVertexCoords[Index].Z) + X1 * BindingGroup.RegionVertexCoords[Index].Z;

				BoundPositions[BindingGroup.RegionVertices[Index]] = FVector3f(X);
			}
		}
	}
}