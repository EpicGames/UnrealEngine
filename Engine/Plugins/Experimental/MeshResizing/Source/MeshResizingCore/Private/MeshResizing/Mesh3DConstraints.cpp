// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/Mesh3DConstraints.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Chaos/TriangleMesh.h"

namespace UE::MeshResizing
{
	static bool DO_NAN_CHECK = true;
	static double LENGTH_CHECK = 2.;
	using namespace Chaos::Softs;

	FShearConstraint::FShearConstraint(float ShearConstraintStrength, const TArray<float>& ShearConstraintWeights, const int32 InNumParticles)
	: NumParticles(InNumParticles)
	, ShearWeightMap(Chaos::Softs::FSolverVec2(0.f, ShearConstraintStrength), ShearConstraintWeights, NumParticles)
	{
		if (!ShearWeightMap.HasWeightMap())
		{
			// Treat as constant EdgeRotationStrength rather than constant 0
			ShearWeightMap.SetWeightedValue(Chaos::Softs::FSolverVec2(ShearConstraintStrength, ShearConstraintStrength));
		}
	}

	void FShearConstraint::Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const UE::Geometry::FDynamicMesh3& InitialResizedMesh, const UE::Geometry::FDynamicMesh3& BaseMesh) const
	{
		check(ResizedMesh.MaxVertexID() <= NumParticles);
		using namespace UE::Geometry;
		using namespace Chaos::Softs;

		// Do a Jacobi update
		UE::Geometry::FDynamicMesh3 ResizedMesh0;
		ResizedMesh0.Copy(ResizedMesh);

		for (int32 TriIndex : ResizedMesh.TriangleIndicesItr())
		{
			const FIndex3i& Tri = ResizedMesh.GetTriangleRef(TriIndex);
			if (!ensure(BaseMesh.IsVertex(Tri[0]) && BaseMesh.IsVertex(Tri[1]) && BaseMesh.IsVertex(Tri[2])))
			{
				continue;
			}
			const float Shear0 = ShearWeightMap.GetValue(Tri[0]);
			const float Shear1 = ShearWeightMap.GetValue(Tri[1]);
			const float Shear2 = ShearWeightMap.GetValue(Tri[2]);
			if (Shear0 == 0.f && Shear1 == 0.f && Shear2 == 0.f)
			{
				continue;
			}

			// calculate the transformation between the current resized mesh and the base mesh.
			// Using Chaos notation of P = deforming positions, X = base positions
			const FVector3d& X0 = BaseMesh.GetVertexRef(Tri[0]);
			const FVector3d& X1 = BaseMesh.GetVertexRef(Tri[1]);
			const FVector3d& X2 = BaseMesh.GetVertexRef(Tri[2]);

			const FVector3d X10 = X1 - X0;
			const FVector3d X20 = X2 - X0;
			const FVector3d X10xX20 = FVector3d::CrossProduct(X10, X20).GetSafeNormal(UE_SMALL_NUMBER);

			// If the base triangle is degenerate, we won't be able to find the transformation.
			if (X10xX20.IsNearlyZero())
			{
				continue;
			}

			const FVector3d P0 = ResizedMesh0.GetVertex(Tri[0]);
			const FVector3d P1 = ResizedMesh0.GetVertex(Tri[1]);
			const FVector3d P2 = ResizedMesh0.GetVertex(Tri[2]);
			const FVector3d P10 = P1 - P0;
			const FVector3d P20 = P2 - P0;
			const FVector3d P10xP20 = FVector3d::CrossProduct(P10, P20).GetSafeNormal(UE_SMALL_NUMBER);
			if (P10xP20.IsNearlyZero())
			{
				continue;
			}
			const FMatrix3d PMat(P10, P20, P10xP20, false);
			const FMatrix3d XMat(X10, X20, X10xX20, false);
			const FMatrix3d XMatInv = XMat.Inverse();
			const FMatrix3d TransformMat = PMat * XMatInv;
			const FMatrix3d TransformMatT = TransformMat.Transpose();

			TMatrix<double> TransformMat44(TransformMatT.Row0, TransformMatT.Row1, TransformMatT.Row2, FVector3d::ZeroVector);

			check(TransformMat44.TransformPosition(X10).Equals(P10));
			check(TransformMat44.TransformPosition(X20).Equals(P20));
			check(TransformMat44.TransformPosition(X10xX20).Equals(P10xP20));

			TVector<double> Scale = TransformMat44.ExtractScaling();
			if (TransformMat44.Determinant() < 0.f)
			{
				// Assume it is along X and modify transform accordingly. 
				// It doesn't actually matter which axis we choose, the 'appearance' will be the same
				Scale.X *= -1.f;
				TransformMat44.SetAxis(0, -TransformMat44.GetScaledAxis(EAxis::X));
			}

			TQuat<double> InRotation = TQuat<double>(TransformMat44);


			const FVector3d PInit0 = InitialResizedMesh.GetVertex(Tri[0]);
			const FVector3d PInit1 = InitialResizedMesh.GetVertex(Tri[1]);
			const FVector3d PInit2 = InitialResizedMesh.GetVertex(Tri[2]);

			FVector3d P10New = X10.GetSafeNormal() * (PInit1 - PInit0).Length();// *Scale;
			P10New = InRotation.RotateVector(P10New);
			check(!DO_NAN_CHECK || !P10New.ContainsNaN());

			FVector3d P20New = X20.GetSafeNormal() * (PInit2 - PInit0).Length();// *Scale;
			P20New = InRotation.RotateVector(P20New);
			check(!DO_NAN_CHECK || !P20New.ContainsNaN());
			const FVector3d Delta20 = P20New - P20;
			const FVector3d Delta10 = P10New - P10;

			//check(!DO_NAN_CHECK || Delta20.SquaredLength() < LENGTH_CHECK);
			//check(!DO_NAN_CHECK || Delta10.SquaredLength() < LENGTH_CHECK);
			const FVector3d P1New = ResizedMesh.GetVertex(Tri[1]) + Shear1 * Delta10;
			const FVector3d P2New = ResizedMesh.GetVertex(Tri[2]) + Shear2 * Delta20;
			const FVector3d P0New = ResizedMesh.GetVertex(Tri[0]) - Shear0 * (Delta10 + Delta20);

			ResizedMesh.SetVertex(Tri[0], P0New);
			ResizedMesh.SetVertex(Tri[1], P1New);
			ResizedMesh.SetVertex(Tri[2], P2New);
		}
	}

	void FShearConstraint::Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const UE::Geometry::FDynamicMesh3& InitialResizedMesh, const UE::Geometry::FDynamicMesh3& BaseMesh, const TArray<float>& InvMass) const
	{
		if (!ensure(InvMass.Num() == NumParticles))
		{
			return;
		}
		check(ResizedMesh.MaxVertexID() <= NumParticles);
		using namespace UE::Geometry;
		using namespace Chaos::Softs;

		// Do a Gauss-Seidel update

		for (int32 TriIndex : ResizedMesh.TriangleIndicesItr())
		{
			const FIndex3i& Tri = ResizedMesh.GetTriangleRef(TriIndex);
			if (!ensure(BaseMesh.IsVertex(Tri[0]) && BaseMesh.IsVertex(Tri[1]) && BaseMesh.IsVertex(Tri[2])))
			{
				continue;
			}
			const float Shear0 = ShearWeightMap.GetValue(Tri[0]);
			const float Shear1 = ShearWeightMap.GetValue(Tri[1]);
			const float Shear2 = ShearWeightMap.GetValue(Tri[2]);
			if (Shear0 == 0.f && Shear1 == 0.f && Shear2 == 0.f)
			{
				continue;
			}

			// calculate the transformation between the current resized mesh and the base mesh.
			// Using Chaos notation of P = deforming positions, X = base positions
			const FVector3d& X0 = BaseMesh.GetVertexRef(Tri[0]);
			const FVector3d& X1 = BaseMesh.GetVertexRef(Tri[1]);
			const FVector3d& X2 = BaseMesh.GetVertexRef(Tri[2]);

			const FVector3d X10 = X1 - X0;
			const FVector3d X20 = X2 - X0;
			const FVector3d X10xX20 = FVector3d::CrossProduct(X10, X20).GetSafeNormal(UE_SMALL_NUMBER);

			// If the base triangle is degenerate, we won't be able to find the transformation.
			if (X10xX20.IsNearlyZero())
			{
				continue;
			}

			const FVector3d P0 = ResizedMesh.GetVertex(Tri[0]);
			const FVector3d P1 = ResizedMesh.GetVertex(Tri[1]);
			const FVector3d P2 = ResizedMesh.GetVertex(Tri[2]);
			const FVector3d P10 = P1 - P0;
			const FVector3d P20 = P2 - P0;
			const FVector3d P10xP20 = FVector3d::CrossProduct(P10, P20).GetSafeNormal(UE_SMALL_NUMBER);
			if (P10xP20.IsNearlyZero())
			{
				continue;
			}
			const FMatrix3d PMat(P10, P20, P10xP20, false);
			const FMatrix3d XMat(X10, X20, X10xX20, false);
			const FMatrix3d XMatInv = XMat.Inverse();
			const FMatrix3d TransformMat = PMat * XMatInv;
			const FMatrix3d TransformMatT = TransformMat.Transpose();

			TMatrix<double> TransformMat44(TransformMatT.Row0, TransformMatT.Row1, TransformMatT.Row2, FVector3d::ZeroVector);

			check(TransformMat44.TransformPosition(X10).Equals(P10));
			check(TransformMat44.TransformPosition(X20).Equals(P20));
			check(TransformMat44.TransformPosition(X10xX20).Equals(P10xP20));

			TVector<double> Scale = TransformMat44.ExtractScaling();
			if (TransformMat44.Determinant() < 0.f)
			{
				// Assume it is along X and modify transform accordingly. 
				// It doesn't actually matter which axis we choose, the 'appearance' will be the same
				Scale.X *= -1.f;
				TransformMat44.SetAxis(0, -TransformMat44.GetScaledAxis(EAxis::X));
			}

			TQuat<double> InRotation = TQuat<double>(TransformMat44);


			const FVector3d PInit0 = InitialResizedMesh.GetVertex(Tri[0]);
			const FVector3d PInit1 = InitialResizedMesh.GetVertex(Tri[1]);
			const FVector3d PInit2 = InitialResizedMesh.GetVertex(Tri[2]);

			FVector3d P10New = X10.GetSafeNormal() * (PInit1 - PInit0).Length();// *Scale;
			P10New = InRotation.RotateVector(P10New);
			check(!DO_NAN_CHECK || !P10New.ContainsNaN());

			FVector3d P20New = X20.GetSafeNormal() * (PInit2 - PInit0).Length();// *Scale;
			P20New = InRotation.RotateVector(P20New);
			check(!DO_NAN_CHECK || !P20New.ContainsNaN());
			const FSolverReal InvM0 = InvMass[Tri[0]];
			const FSolverReal InvM1 = InvMass[Tri[1]];
			const FSolverReal InvM2 = InvMass[Tri[2]];
			if (InvM2 != (FSolverReal)0. || InvM0 != (FSolverReal)0.)
			{
				const FVector3d Delta20 = (P20New - P20) / (InvM2 + InvM0);
				ResizedMesh.SetVertex(Tri[0], ResizedMesh.GetVertex(Tri[0]) - Shear0 * Delta20 * InvM0);
				ResizedMesh.SetVertex(Tri[2], ResizedMesh.GetVertex(Tri[2]) + Shear2 * Delta20 * InvM2);
			}
			if (InvM1 != (FSolverReal)0. || InvM0 != (FSolverReal)0.)
			{
				const FVector3d Delta10 = (P10New - P10) / (InvM1 + InvM0);
				ResizedMesh.SetVertex(Tri[0], ResizedMesh.GetVertex(Tri[0]) - Shear0 * Delta10 * InvM0);
				ResizedMesh.SetVertex(Tri[1], ResizedMesh.GetVertex(Tri[1]) + Shear1 * Delta10 * InvM1);
			}
		}
	}

	FEdgeConstraint::FEdgeConstraint(float EdgeConstraintStrength, const TArray<float>& EdgeConstraintWeights, const int32 InNumParticles)
		: NumParticles(InNumParticles)
		, EdgeWeightMap(Chaos::Softs::FSolverVec2(0.f, EdgeConstraintStrength), EdgeConstraintWeights, NumParticles)
	{
		if (!EdgeWeightMap.HasWeightMap())
		{
			// Treat as constant EdgeRotationStrength rather than constant 0
			EdgeWeightMap.SetWeightedValue(Chaos::Softs::FSolverVec2(EdgeConstraintStrength, EdgeConstraintStrength));
		}
	}

	void FEdgeConstraint::Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const UE::Geometry::FDynamicMesh3& InitialResizedMesh, const UE::Geometry::FDynamicMesh3& BaseMesh, const TArray<float>& InvMass) const
	{
		if (!ensure(InvMass.Num() == NumParticles))
		{
			return;
		}
		check(ResizedMesh.MaxVertexID() <= NumParticles);
		using namespace UE::Geometry;

		// Do a Gauss-Seidel update

		for (int32 EdgeIndex : ResizedMesh.EdgeIndicesItr())
		{
			const FDynamicMesh3::FEdge& Edge = ResizedMesh.GetEdgeRef(EdgeIndex);
			int VertIndex0 = Edge.Vert[0];
			int VertIndex1 = Edge.Vert[1];
			if (!ensure(ResizedMesh.IsVertex(VertIndex0) && ResizedMesh.IsVertex(VertIndex1) && 
				InvMass.IsValidIndex(VertIndex0) && InvMass.IsValidIndex(VertIndex1)))
			{
				continue;
			}
			const FSolverReal EdgeWeight0 = EdgeWeightMap.GetValue(VertIndex0);
			const FSolverReal EdgeWeight1 = EdgeWeightMap.GetValue(VertIndex1);
			if (EdgeWeight0 == 0.f && EdgeWeight1 == 0.f)
			{
				continue;
			}

			const FVector3d P0 = ResizedMesh.GetVertex(VertIndex0);
			const FVector3d P1 = ResizedMesh.GetVertex(VertIndex1);
			const FSolverReal InvM0 = InvMass[VertIndex0];
			const FSolverReal InvM1 = InvMass[VertIndex1];

			if (InvM0 == (FSolverReal)0. && InvM1 == (FSolverReal)0.)
			{
				continue;
			}
			const FSolverReal CombinedInvMass = InvM0 + InvM1;
			const float InitialResizedLength = (InitialResizedMesh.GetVertex(VertIndex0) - InitialResizedMesh.GetVertex(VertIndex1)).Size();
			const FVector3d Direction = (P0 - P1).GetSafeNormal();
			const double Distance = (P0 - P1).Size();

			const FVector3d Delta = (Distance - InitialResizedLength) * Direction / CombinedInvMass;
			ResizedMesh.SetVertex(VertIndex0, P0 - InvM0 * Delta * EdgeWeight0);
			ResizedMesh.SetVertex(VertIndex1, P1 + InvM1 * Delta * EdgeWeight1);
		}
	}

	template<class TNum>
	static TNum SafeDivide(const TNum& Numerator, const FSolverReal& Denominator)
	{
		if (Denominator > SMALL_NUMBER)
			return Numerator / Denominator;
		return TNum(0);
	}

	static TStaticArray<FVector3d, 4> GetGradients(const UE::Geometry::FDynamicMesh3& Mesh, const Chaos::TVec4<int32>& Constraint)
	{
		const FVector3d P1 = Mesh.GetVertex(Constraint[0]);
		const FVector3d P2 = Mesh.GetVertex(Constraint[1]);
		const FVector3d P3 = Mesh.GetVertex(Constraint[2]);
		const FVector3d P4 = Mesh.GetVertex(Constraint[3]);

		TStaticArray<FVector3d, 4> Grads;
		// Calculated using Phi = atan2(SinPhi, CosPhi)
		// where SinPhi = (Normal1 ^ Normal2)*SharedEdgeNormalized, CosPhi = Normal1 * Normal2
		// Full gradients are calculated here, i.e., no simplifying assumptions around things like edge lengths being constant.
		const FVector3d SharedEdgeNormalized = (P2 - P1).GetSafeNormal();
		const FVector3d P13CrossP23 = FVector3d::CrossProduct(P1 - P3, P2 - P3);
		const FSolverReal Normal1Len = P13CrossP23.Size();
		const FVector3d Normal1 = P13CrossP23.GetSafeNormal();
		const FVector3d P24CrossP14 = FVector3d::CrossProduct(P2 - P4, P1 - P4);
		const FSolverReal Normal2Len = P24CrossP14.Size();
		const FVector3d Normal2 = P24CrossP14.GetSafeNormal();

		const FVector3d N2CrossN1 = FVector3d::CrossProduct(Normal2, Normal1);

		const FSolverReal CosPhi = FMath::Clamp(FVector3d::DotProduct(Normal1, Normal2), (FSolverReal)-1, (FSolverReal)1);
		const FSolverReal SinPhi = FMath::Clamp(FVector3d::DotProduct(N2CrossN1, SharedEdgeNormalized), (FSolverReal)-1, (FSolverReal)1);

		const FVector3d DPhiDN1_OverNormal1Len = SafeDivide(CosPhi * FVector3d::CrossProduct(SharedEdgeNormalized, Normal2) - SinPhi * Normal2, Normal1Len);
		const FVector3d DPhiDN2_OverNormal2Len = SafeDivide(CosPhi * FVector3d::CrossProduct(Normal1, SharedEdgeNormalized) - SinPhi * Normal1, Normal2Len);

		const FVector3d DPhiDP13 = FVector3d::CrossProduct(P2 - P3, DPhiDN1_OverNormal1Len);
		const FVector3d DPhiDP23 = FVector3d::CrossProduct(DPhiDN1_OverNormal1Len, P1 - P3);
		const FVector3d DPhiDP24 = FVector3d::CrossProduct(P1 - P4, DPhiDN2_OverNormal2Len);
		const FVector3d DPhiDP14 = FVector3d::CrossProduct(DPhiDN2_OverNormal2Len, P2 - P4);

		Grads[0] = DPhiDP13 + DPhiDP14;
		Grads[1] = DPhiDP23 + DPhiDP24;
		Grads[2] = -DPhiDP13 - DPhiDP23;
		Grads[3] = -DPhiDP14 - DPhiDP24;
		return Grads;
	}

	static FSolverReal CalcAngle(const FVector3d& P1, const FVector3d& P2, const FVector3d& P3, const FVector3d& P4)
	{
		const FVector3d Normal1 = FVector3d::CrossProduct(P1 - P3, P2 - P3).GetSafeNormal();
		const FVector3d Normal2 = FVector3d::CrossProduct(P2 - P4, P1 - P4).GetSafeNormal();

		const FVector3d SharedEdge = (P2 - P1).GetSafeNormal();

		const FSolverReal CosPhi = FMath::Clamp(FVector3d::DotProduct(Normal1, Normal2), (FSolverReal)-1, (FSolverReal)1);
		const FSolverReal SinPhi = FMath::Clamp(FVector3d::DotProduct(FVector3d::CrossProduct(Normal2, Normal1), SharedEdge), (FSolverReal)-1, (FSolverReal)1);
		return FMath::Atan2(SinPhi, CosPhi);
	}

	FBendingConstraint::FBendingConstraint(const UE::Geometry::FDynamicMesh3& BaseMesh, float BendingConstraintStrength, const TArray<float>& BendingConstraintWeights, const int32 InNumParticles)
		: NumParticles(InNumParticles)
		, BendingConstraintWeightMap(Chaos::Softs::FSolverVec2(0.f, BendingConstraintStrength), BendingConstraintWeights, NumParticles)
	{
		if (!BendingConstraintWeightMap.HasWeightMap())
		{
			// Treat as constant EdgeRotationStrength rather than constant 0
			BendingConstraintWeightMap.SetWeightedValue(Chaos::Softs::FSolverVec2(BendingConstraintStrength, BendingConstraintStrength));
		}
		//Create constraints
		Chaos::FTriangleMesh TriangleMesh;
		TArray<Chaos::TVec3<int32>> Elements;
		for (int32 TriIndex : BaseMesh.TriangleIndicesItr())
		{
			Elements.Add(
				{ BaseMesh.GetTriangleRef(TriIndex)[0],
				  BaseMesh.GetTriangleRef(TriIndex)[1],
				  BaseMesh.GetTriangleRef(TriIndex)[2]});
		}

		TriangleMesh.Init(MoveTemp(Elements), 0, NumParticles - 1);
		Constraints = TriangleMesh.GetUniqueAdjacentElements();

		//Calculate rest angle
		RestAngles.Reset(Constraints.Num());
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			const Chaos::TVec4<int32>& Constraint = Constraints[ConstraintIndex];
			const FVector3d& P1 = BaseMesh.GetVertex(Constraint[0]);
			const FVector3d& P2 = BaseMesh.GetVertex(Constraint[1]);
			const FVector3d& P3 = BaseMesh.GetVertex(Constraint[2]);
			const FVector3d& P4 = BaseMesh.GetVertex(Constraint[3]);
			RestAngles.Add(FMath::Clamp(CalcAngle(P1, P2, P3, P4), -UE_PI, UE_PI));
		}
	}

	FSolverReal FBendingConstraint::GetScalingFactor(const UE::Geometry::FDynamicMesh3& Mesh, int32 ConstraintIndex, const TStaticArray<FVector3d, 4>& Grads, const FSolverReal ExpStiffnessValue, const TArray<float>& InvMass) const
	{
		const Chaos::TVec4<int32>& Constraint = Constraints[ConstraintIndex];
		const FVector3d P1 = Mesh.GetVertex(Constraint[0]);
		const FVector3d P2 = Mesh.GetVertex(Constraint[1]);
		const FVector3d P3 = Mesh.GetVertex(Constraint[2]);
		const FVector3d P4 = Mesh.GetVertex(Constraint[3]);
		const FSolverReal Angle = CalcAngle(P1, P2, P3, P4);
		const FSolverReal Denom = (InvMass[Constraint[0]] * Grads[0].SizeSquared() + InvMass[Constraint[1]] * Grads[1].SizeSquared() + InvMass[Constraint[2]] * Grads[2].SizeSquared() + InvMass[Constraint[3]] * Grads[3].SizeSquared());

		constexpr FSolverReal SingleStepAngleLimit = (FSolverReal)(UE_PI * .25f); // this constraint is very non-linear. taking large steps is not accurate
		const FSolverReal Delta = FMath::Clamp(ExpStiffnessValue * (Angle - RestAngles[ConstraintIndex]), -SingleStepAngleLimit, SingleStepAngleLimit);
		return SafeDivide(Delta, Denom);
	}

	void FBendingConstraint::Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const TArray<float>& InvMass) const
	{
		if (!ensure(InvMass.Num() == NumParticles))
		{
			return;
		}
		check(ResizedMesh.MaxVertexID() <= NumParticles);
		using namespace UE::Geometry;

		// Do a Gauss-Seidel update
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			const Chaos::TVec4<int32>& Constraint = Constraints[ConstraintIndex];
			const int32 i1 = Constraint[0];
			const int32 i2 = Constraint[1];
			const int32 i3 = Constraint[2];
			const int32 i4 = Constraint[3];
			const TStaticArray<FVector3d, 4> Grads = GetGradients(ResizedMesh, Constraint);
			const FSolverReal StiffnessValue = (BendingConstraintWeightMap.GetValue(i1) + BendingConstraintWeightMap.GetValue(i2) + BendingConstraintWeightMap.GetValue(i3) + BendingConstraintWeightMap.GetValue(i4)) / 4;
			const FSolverReal S = GetScalingFactor(ResizedMesh, ConstraintIndex, Grads, StiffnessValue, InvMass);

			ResizedMesh.SetVertex(i1, ResizedMesh.GetVertex(i1) - S * InvMass[i1] * Grads[0]);
			ResizedMesh.SetVertex(i2, ResizedMesh.GetVertex(i2) - S * InvMass[i2] * Grads[1]);
			ResizedMesh.SetVertex(i3, ResizedMesh.GetVertex(i3) - S * InvMass[i3] * Grads[2]);
			ResizedMesh.SetVertex(i4, ResizedMesh.GetVertex(i4) - S * InvMass[i4] * Grads[3]);
		}
	}

	FExternalForceConstraint::FExternalForceConstraint(const TArray<FVector3d>& InParticleExternalForce, const int32 InNumParticles)
		: NumParticles(InNumParticles)
		, ParticleExternalForce(InParticleExternalForce) {}

	void FExternalForceConstraint::Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const TArray<float>& InvMass) const
	{
		if (!ensure(InvMass.Num() == NumParticles))
		{
			return;
		}
		check(ResizedMesh.MaxVertexID() <= NumParticles);
		using namespace UE::Geometry;

		for (int32 VertexIndex : ResizedMesh.VertexIndicesItr())
		{
			if (!ensure(ResizedMesh.IsVertex(VertexIndex) && InvMass.IsValidIndex(VertexIndex) && ParticleExternalForce.IsValidIndex(VertexIndex)))
			{
				continue;
			}

			const FVector3d P0 = ResizedMesh.GetVertex(VertexIndex);
			const FSolverReal InvM0 = InvMass[VertexIndex];

			ResizedMesh.SetVertex(VertexIndex, P0 + InvM0 * ParticleExternalForce[VertexIndex]);
		}
	}
}