// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

namespace UE::MeshResizing
{
	class FMeshResizingCoreModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
		}

		virtual void ShutdownModule() override
		{
		}
	};
}

IMPLEMENT_MODULE(UE::MeshResizing::FMeshResizingCoreModule, MeshResizingCore);
