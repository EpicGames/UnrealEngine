// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/RBFInterpolation.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "GroomBindingBuilder.h"
#include "GroomRBFDeformer.h"
#include "Tasks/Task.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RBFInterpolation)

namespace UE::MeshResizing
{
	namespace Private
	{
		static void UpdateInterpolationData(const GroomBinding_RBFWeighting::FPointsSampler& PointsSampler, const GroomBinding_RBFWeighting::FWeightsBuilder& InterpolationWeights, FMeshResizingRBFInterpolationData& OutData)
		{
			OutData.SampleIndices.SetNumUninitialized(PointsSampler.SampleIndices.Num());
			for (int32 Index = 0; Index < PointsSampler.SampleIndices.Num(); ++Index)
			{
				OutData.SampleIndices[Index] = (int32)PointsSampler.SampleIndices[Index];
			}

			OutData.SampleRestPositions = PointsSampler.SamplePositions;
			OutData.InterpolationWeights = InterpolationWeights.InverseEntries;
		}

		static FVector3f DeformPoint(const FMeshResizingRBFInterpolationData& InterpolationData, const TArray<FVector3f>& SampleDeformations, const FVector3f& RestControlPoint)
		{
			const int32 SampleCount = InterpolationData.SampleRestPositions.Num();
			check(SampleCount + 4 == SampleDeformations.Num());

			FVector3f ControlPoint = RestControlPoint;
			for (int32 SampleIndex = 0; SampleIndex < SampleCount; ++SampleIndex)
			{
				const FVector3f PositionDelta = RestControlPoint - InterpolationData.SampleRestPositions[SampleIndex];
				const float FunctionValue = FMath::Sqrt(FVector3f::DotProduct(PositionDelta, PositionDelta) + 1);
				ControlPoint += FunctionValue * SampleDeformations[SampleIndex];
			}
			ControlPoint += SampleDeformations[SampleCount];
			ControlPoint += SampleDeformations[SampleCount + 1] * RestControlPoint.X;
			ControlPoint += SampleDeformations[SampleCount + 2] * RestControlPoint.Y;
			ControlPoint += SampleDeformations[SampleCount + 3] * RestControlPoint.Z;

			return ControlPoint;
		}

		// From GroomRBFDeformer
		static void UpdateMeshSamples(const TArray<FVector3f>& TargetPositions, const FMeshResizingRBFInterpolationData& InterpolationData, TArray<FVector3f>& OutSampleDeformations)
		{
			const int32 MaxSampleCount = TargetPositions.Num();
			check(MaxSampleCount == InterpolationData.SampleIndices.Num());
			check(MaxSampleCount == InterpolationData.SampleRestPositions.Num());

			const int32 EntryCount = FGroomRBFDeformer::GetEntryCount(MaxSampleCount);
			OutSampleDeformations.SetNum(EntryCount);

			const int32 NumTasks = FMath::Max(FMath::Min(int32(FTaskGraphInterface::Get().GetNumWorkerThreads()), EntryCount), 1);
			constexpr int32 MinEntryByTask = 10;
			const int32 EntryByTask = FMath::Max(FMath::DivideAndRoundUp(EntryCount, NumTasks), MinEntryByTask);
			const int32 NumBatches = FMath::DivideAndRoundUp(EntryCount, EntryByTask);
			TArray<UE::Tasks::FTask> PendingTasks;
			PendingTasks.Reserve(NumBatches);

			for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
			{
				const int32 StartIndex = BatchIndex * EntryByTask;
				int32 EndIndex = (BatchIndex + 1) * EntryByTask;
				EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(EntryCount, EndIndex) : EndIndex;

				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [StartIndex, EndIndex, EntryCount, MaxSampleCount, &OutSampleDeformations, &InterpolationData, &TargetPositions]
				{
					for (int32 SampleIndex = StartIndex; SampleIndex < EndIndex; ++SampleIndex)
					{
						int32 WeightsOffset = SampleIndex * EntryCount;
						FVector3f SampleDeformation(FVector3f::ZeroVector);
						for (int32 Index = 0; Index < MaxSampleCount; ++Index, ++WeightsOffset)
						{
							SampleDeformation += InterpolationData.InterpolationWeights[WeightsOffset] * (TargetPositions[Index] - InterpolationData.SampleRestPositions[Index]);
						}
						OutSampleDeformations[SampleIndex] = SampleDeformation;
					}
				});
					PendingTasks.Add(PendingTask);
			}
			UE::Tasks::Wait(PendingTasks);
		}

		// From GroomRBFDeformer
		static void ApplyRBFDeformation(const FMeshResizingRBFInterpolationData& InterpolationData, const TArray<FVector3f>& SampleDeformations, bool bInterpolateNormals, UE::Geometry::FDynamicMesh3& DeformingMesh)
		{
			const int32 SampleCount = InterpolationData.SampleRestPositions.Num();
			check(SampleCount + 4 == SampleDeformations.Num());

			TArray<UE::Geometry::FDynamicMeshNormalOverlay*> NormalOverlays;
			if (bInterpolateNormals)
			{
				const int32 NumNormalLayers = DeformingMesh.Attributes() ? DeformingMesh.Attributes()->NumNormalLayers() : 0;
				NormalOverlays.Reserve(NumNormalLayers);

				for (int32 NormalLayerId = 0; NormalLayerId < NumNormalLayers; ++NormalLayerId)
				{
					if (UE::Geometry::FDynamicMeshNormalOverlay* const Overlay = DeformingMesh.Attributes()->GetNormalLayer(NormalLayerId))
					{
						NormalOverlays.Emplace(Overlay);
					}
				}
			}

			auto DeformPointLambda = [&SampleDeformations, &InterpolationData](const FVector3f& RestControlPoint)
			{
				return DeformPoint(InterpolationData, SampleDeformations, RestControlPoint);
			};

			const int32 NumVertexIndices = DeformingMesh.VertexCount();
			const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), NumVertexIndices), 1);
			constexpr int32 MinVerticesByTask = 10;
			const int32 VerticesByTask = FMath::Max(FMath::Max(FMath::DivideAndRoundUp(NumVertexIndices, NumTasks), MinVerticesByTask), 1);
			const int32 NumBatches = FMath::DivideAndRoundUp(NumVertexIndices, VerticesByTask);
			TArray<UE::Tasks::FTask> PendingTasks;
			PendingTasks.Reserve(NumBatches);

			for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
			{
				const int32 StartIndex = BatchIndex * VerticesByTask;
				int32 EndIndex = (BatchIndex + 1) * VerticesByTask;
				EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(NumVertexIndices, EndIndex) : EndIndex;

				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&DeformingMesh, StartIndex, EndIndex, BatchIndex, DeformPointLambda, &NormalOverlays]
				{
					for (int32 VertexIndex = StartIndex; VertexIndex < EndIndex; VertexIndex++)
					{
						check(DeformingMesh.GetVerticesRefCounts().IsValid(VertexIndex));
						const FVector3f RestPoint(FVector3f(DeformingMesh.GetVertexRef(VertexIndex)));
						const FVector3f DeformedPoint = DeformPointLambda(RestPoint);

						DeformingMesh.SetVertex(VertexIndex, FVector3d(DeformedPoint));

						for (UE::Geometry::FDynamicMeshNormalOverlay* const Overlay : NormalOverlays)
						{
							Overlay->EnumerateVertexElements(VertexIndex,
								[&RestPoint, &DeformedPoint, &DeformPointLambda, Overlay](int32 TriangleID, int32 ElementID, const FVector3f& Normal)
								{
									const FVector3f RestNormalEnd = RestPoint + Normal;
									const FVector3f DeformedNormalEnd = DeformPointLambda(RestNormalEnd);
									const FVector3f DeformedNorml = (DeformedNormalEnd - DeformedPoint).GetSafeNormal();
									Overlay->SetElement(ElementID, DeformedNorml);
									return true;
								}
							);
						}
					}
				});
				PendingTasks.Add(PendingTask);
			}
			UE::Tasks::Wait(PendingTasks);
		}

		// From GroomRBFDeformer
		static void ApplyRBFDeformation(const FMeshResizingRBFInterpolationData& InterpolationData, const TArray<FVector3f>& SampleDeformations, TArrayView<FVector3f>& Positions, TArray<TArrayView<FVector3f>> NormalsArray)
		{
			const int32 SampleCount = InterpolationData.SampleRestPositions.Num();
			check(SampleCount + 4 == SampleDeformations.Num());

			auto DeformPointLambda = [&SampleDeformations, &InterpolationData](const FVector3f& RestControlPoint)
				{
					return DeformPoint(InterpolationData, SampleDeformations, RestControlPoint);
				};

			const int32 NumVertexIndices = Positions.Num();
			const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), NumVertexIndices), 1);
			constexpr int32 MinVerticesByTask = 10;
			const int32 VerticesByTask = FMath::Max(FMath::Min(FMath::DivideAndRoundUp(NumVertexIndices, NumTasks), MinVerticesByTask), 1);
			const int32 NumBatches = FMath::DivideAndRoundUp(NumVertexIndices, VerticesByTask);
			TArray<UE::Tasks::FTask> PendingTasks;
			PendingTasks.Reserve(NumBatches);

			for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
			{
				const int32 StartIndex = BatchIndex * VerticesByTask;
				int32 EndIndex = (BatchIndex + 1) * VerticesByTask;
				EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(NumVertexIndices, EndIndex) : EndIndex;

				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&Positions, StartIndex, EndIndex, BatchIndex, DeformPointLambda, &NormalsArray]
					{
						for (int32 VertexIndex = StartIndex; VertexIndex < EndIndex; VertexIndex++)
						{
							const FVector3f RestPoint(Positions[VertexIndex]);
							const FVector3f DeformedPoint = DeformPointLambda(RestPoint);

							Positions[VertexIndex] = DeformedPoint;

							for (TArrayView<FVector3f>& Normals : NormalsArray)
							{								
								const FVector3f RestNormalEnd = RestPoint + Normals[VertexIndex];
								const FVector3f DeformedNormalEnd = DeformPointLambda(RestNormalEnd);
								Normals[VertexIndex] = (DeformedNormalEnd - DeformedPoint).GetSafeNormal();
							}
						}
					});
				PendingTasks.Add(PendingTask);
			}
			UE::Tasks::Wait(PendingTasks);
		}
	}

	void FRBFInterpolation::GenerateWeights(const UE::Geometry::FDynamicMesh3& BaseMesh, int32 NumInterpolationPoints, FMeshResizingRBFInterpolationData& OutData)
	{
		// Groom binding sampler expects points to be a contiguous array of FVector3f. Just copy that way for now.
		TArray<FVector3f> Positions;
		Positions.Reserve(BaseMesh.VertexCount());
		for (const int32 VertexIndex : BaseMesh.VertexIndicesItr())
		{
			Positions.Emplace(FVector3f(BaseMesh.GetVertexRef(VertexIndex)));
		}

		GenerateWeights(TConstArrayView<FVector3f>(Positions), NumInterpolationPoints, OutData);
	}

	void FRBFInterpolation::GenerateWeights(const FMeshDescription& BaseMesh, int32 NumInterpolationPoints, FMeshResizingRBFInterpolationData& OutData)
	{
		// Groom binding sampler expects points to be a contiguous array of FVector3f. Just copy that way for now.
		TArray<FVector3f> Positions;
		TArray<int32> CompactToVertID;
		Positions.Reserve(BaseMesh.Vertices().Num());
		CompactToVertID.Reserve(BaseMesh.Vertices().Num());
		bool bCompactMatchesVertID = true;
		for (const int32 VertexIndex : BaseMesh.Vertices().GetElementIDs())
		{
			Positions.Emplace(BaseMesh.GetVertexPosition(VertexIndex));
			const int32 CompactIndex = CompactToVertID.Emplace(VertexIndex);
			bCompactMatchesVertID = bCompactMatchesVertID && (CompactIndex == VertexIndex);
		}

		GenerateWeights(TConstArrayView<FVector3f>(Positions), NumInterpolationPoints, OutData);
		if (!bCompactMatchesVertID)
		{
			// Update SampleIndices to be the true VertexIDs, not the CompactIDs
			for (int32& SampleIndex : OutData.SampleIndices)
			{
				SampleIndex = CompactToVertID[SampleIndex];
			}
		}
	}

	void FRBFInterpolation::GenerateWeights(const TConstArrayView<FVector3f>& SourcePositions, int32 NumInterpolationPoints, FMeshResizingRBFInterpolationData& OutData)
	{
		// Consider all points to be valid for now
		TArray<bool> ValidPoints;
		ValidPoints.Init(true, SourcePositions.Num());

		GroomBinding_RBFWeighting::FPointsSampler PointsSampler(ValidPoints, SourcePositions.GetData(), NumInterpolationPoints);
		const int32 SampleCount = PointsSampler.SamplePositions.Num();
		GroomBinding_RBFWeighting::FWeightsBuilder InterpolationWeights(SampleCount, SampleCount, PointsSampler.SamplePositions.GetData(), PointsSampler.SamplePositions.GetData());
		Private::UpdateInterpolationData(PointsSampler, InterpolationWeights, OutData);
	}

	void FRBFInterpolation::GenerateMeshSamples(const TArray<FVector3f>& TargetPositions, const FMeshResizingRBFInterpolationData& InterpolationData, TArray<FVector3f>& SampleDeformations)
	{
		Private::UpdateMeshSamples(TargetPositions, InterpolationData, SampleDeformations);
	}

	void FRBFInterpolation::DeformPoints(const TArray<FVector3f>& TargetPositions, const FMeshResizingRBFInterpolationData& InterpolationData, bool bInterpolateNormals, UE::Geometry::FDynamicMesh3& DeformingMesh)
	{
		TArray<FVector3f> SampleDeformations;
		Private::UpdateMeshSamples(TargetPositions, InterpolationData, SampleDeformations);
		Private::ApplyRBFDeformation(InterpolationData, SampleDeformations, bInterpolateNormals, DeformingMesh);
	}

	void FRBFInterpolation::DeformCoordinateFrames(const TArray<FVector3f>& TargetPositions, const FMeshResizingRBFInterpolationData& InterpolationData, bool bNormalize, bool bOrthogonalize, TArray<FMatrix44f>& Coordinates)
	{
		TArray<FVector3f> SampleDeformations;
		Private::UpdateMeshSamples(TargetPositions, InterpolationData, SampleDeformations);

		for (FMatrix44f& Coord : Coordinates)
		{
			FVector3f Origin = Coord.GetOrigin();
			FVector3f TangentU, TangentV, Normal;
			Coord.GetScaledAxes(TangentU, TangentV, Normal);

			FVector3f TangentUEnd = Origin + TangentU;
			FVector3f TangentVEnd = Origin + TangentV;
			FVector3f NormalEnd = Origin + Normal;
			Origin = Private::DeformPoint(InterpolationData, SampleDeformations, Origin);
			TangentUEnd = Private::DeformPoint(InterpolationData, SampleDeformations, TangentUEnd);
			TangentVEnd = Private::DeformPoint(InterpolationData, SampleDeformations, TangentVEnd);
			NormalEnd = Private::DeformPoint(InterpolationData, SampleDeformations, NormalEnd);

			TangentU = TangentUEnd - Origin;
			TangentV = TangentVEnd - Origin;
			Normal = NormalEnd - Origin;

			if (bOrthogonalize)
			{
				if (bNormalize)
				{
					Normal = Normal.GetSafeNormal();
					TangentU = (TangentU - (TangentU | Normal) * Normal).GetSafeNormal();
					TangentV = Normal.Cross(TangentU);
				}
				else
				{
					const float NormalLenSq = Normal.SquaredLength();
					if (NormalLenSq > UE_SMALL_NUMBER)
					{
						TangentU = (TangentU - (TangentU | Normal) * Normal / NormalLenSq);
					}
					TangentV = (TangentV.Length() / (FMath::Sqrt(NormalLenSq) * TangentU.Length())) * Normal.Cross(TangentU);
				}
			}
			else if (bNormalize)
			{
				Normal = Normal.GetSafeNormal();
				TangentU = TangentU.GetSafeNormal();
				TangentV = TangentV.GetSafeNormal();
			}

			Coord.SetAxes(&TangentU, &TangentV, &Normal, &Origin);
		}
	}

	void FRBFInterpolation::DeformPoints(const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, TArray<FVector3d>& Points)
	{
		TArray<FVector3f> TargetPositions;
		TargetPositions.Reserve(InterpolationData.SampleIndices.Num());
		for (int32 Index = 0; Index < InterpolationData.SampleIndices.Num(); ++Index)
		{
			const int32 SampleIndex = InterpolationData.SampleIndices[Index];
			if (ensure(TargetMesh.IsVertexValid(SampleIndex)))
			{
				TargetPositions.Emplace(TargetMesh.GetVertexPosition(SampleIndex));
			}
			else
			{
				// Just use Source positions
				TargetPositions.Emplace(InterpolationData.SampleRestPositions[Index]);
			}
		}

		TArray<FVector3f> SampleDeformations;
		Private::UpdateMeshSamples(TargetPositions, InterpolationData, SampleDeformations);
		for (FVector3d& Point : Points)
		{
			Point = FVector3d(Private::DeformPoint(InterpolationData, SampleDeformations, FVector3f(Point)));
		}
	}

	void FRBFInterpolation::DeformPoints(const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, bool bInterpolateNormals, UE::Geometry::FDynamicMesh3& DeformingMesh)
	{
		TArray<FVector3f> TargetPositions;
		TargetPositions.Reserve(InterpolationData.SampleIndices.Num());
		for(int32 Index = 0; Index < InterpolationData.SampleIndices.Num(); ++Index)
		{
			const int32 SampleIndex = InterpolationData.SampleIndices[Index];
			if (ensure(TargetMesh.IsVertexValid(SampleIndex)))
			{
				TargetPositions.Emplace(TargetMesh.GetVertexPosition(SampleIndex));
			}
			else
			{
				// Just use Source positions
				TargetPositions.Emplace(InterpolationData.SampleRestPositions[Index]);
			}
		}

		DeformPoints(TargetPositions, InterpolationData, bInterpolateNormals, DeformingMesh);
	}

	void FRBFInterpolation::DeformPoints(const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, TArrayView<FVector3f> Points, TArrayView<FVector3f> Normals)
	{
		TArray<FVector3f> TargetPositions;
		TargetPositions.Reserve(InterpolationData.SampleIndices.Num());
		for (int32 Index = 0; Index < InterpolationData.SampleIndices.Num(); ++Index)
		{
			const int32 SampleIndex = InterpolationData.SampleIndices[Index];
			if (ensure(TargetMesh.IsVertexValid(SampleIndex)))
			{
				TargetPositions.Emplace(TargetMesh.GetVertexPosition(SampleIndex));
			}
			else
			{
				// Just use Source positions
				TargetPositions.Emplace(InterpolationData.SampleRestPositions[Index]);
			}
		}

		TArray<FVector3f> SampleDeformations;
		Private::UpdateMeshSamples(TargetPositions, InterpolationData, SampleDeformations); 
		Private::ApplyRBFDeformation(InterpolationData, SampleDeformations, Points, { Normals });
	}

	void FRBFInterpolation::DeformPoints(const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, TArrayView<FVector3f> Points, TArrayView<FVector3f> Normals, TArrayView<FVector3f> TangentUs, TArrayView<FVector3f> TangentVs)
	{
		TArray<FVector3f> TargetPositions;
		TargetPositions.Reserve(InterpolationData.SampleIndices.Num());
		for (int32 Index = 0; Index < InterpolationData.SampleIndices.Num(); ++Index)
		{
			const int32 SampleIndex = InterpolationData.SampleIndices[Index];
			if (ensure(TargetMesh.IsVertexValid(SampleIndex)))
			{
				TargetPositions.Emplace(TargetMesh.GetVertexPosition(SampleIndex));
			}
			else
			{
				// Just use Source positions
				TargetPositions.Emplace(InterpolationData.SampleRestPositions[Index]);
			}
		}

		TArray<FVector3f> SampleDeformations;
		Private::UpdateMeshSamples(TargetPositions, InterpolationData, SampleDeformations);
		Private::ApplyRBFDeformation(InterpolationData, SampleDeformations, Points, { Normals, TangentUs, TangentVs });
	}
}