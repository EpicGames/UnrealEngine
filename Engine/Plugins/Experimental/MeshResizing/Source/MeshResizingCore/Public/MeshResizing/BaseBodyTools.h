// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 
#include "UObject/NameTypes.h"
#include "Containers/Array.h"

#define UE_API MESHRESIZINGCORE_API

namespace UE::Geometry
{
	class FDynamicMesh3;
}
namespace UE::MeshResizing
{
	struct FBaseBodyTools
	{
		static UE_API const FName ImportedVertexVIDsAttrName;
		static UE_API const FName RawPointIndicesVIDsAttrName;

		/** @return success */
		static UE_API bool AttachVertexMappingData(const FName& AttrName, const TArray<int32>& Data, UE::Geometry::FDynamicMesh3& Mesh);

		/** @return success */
		static UE_API bool GenerateResizableProxyFromVertexMappingData(const UE::Geometry::FDynamicMesh3& SourceMesh, const FName& SourceMappingName, const UE::Geometry::FDynamicMesh3& TargetMesh, const FName& TargetMappingName, UE::Geometry::FDynamicMesh3& ProxyMesh);

		/** @return success */
		static UE_API bool InterpolateResizableProxy(const UE::Geometry::FDynamicMesh3& SourceMesh, const UE::Geometry::FDynamicMesh3& TargetMesh, float BlendAlpha, UE::Geometry::FDynamicMesh3& ProxyMesh);


	};
}

#undef UE_API
