// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 
#include "UObject/NameTypes.h"
#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Math/Matrix.h"

#include "CustomRegionResizing.generated.h"

#define UE_API MESHRESIZINGCORE_API

namespace UE::Geometry
{
	class FDynamicMesh3;
}

UENUM()
enum struct EMeshResizingCustomRegionType : uint8
{
	TrilinearInterpolation,
	/* Hoping to add support for these in a future review
	RigidRivet,
	UniformScaleRivet,
	NonUniformScaleRivet,
	Rivet
	*/
};

USTRUCT()
struct FMeshResizingCustomRegion
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> RegionVertices;
	UPROPERTY()
	TArray<FVector3f> RegionVertexCoords; // Trilinear interpolation within Bounds: (0,0,0) = RegionBoundsCentroid - RegionBoundsExtents; (1,1,1) = RegionBoundsCentroid + RegionBoundsExtents

	UPROPERTY()
	int32 SourceFaceIndex = INDEX_NONE;
	UPROPERTY()
	FVector3f SourceBaryCoords = FVector3f(0.f);
	UPROPERTY()
	FVector3d SourceOrigin = FVector3d(0.f);
	UPROPERTY()
	FVector3f SourceAxis0 = FVector3f(0.f);
	UPROPERTY()
	FVector3f SourceAxis1 = FVector3f(0.f);
	UPROPERTY()
	FVector3f SourceAxis2 = FVector3f(0.f);

	// Relative to Source
	UPROPERTY()
	FVector3f RegionBoundsCentroid = FVector3f(0.f);

	UPROPERTY()
	FVector3f RegionBoundsExtents = FVector3f(0.f);

	void Reset()
	{
		RegionVertices.Reset();
		RegionVertexCoords.Reset();
		SourceFaceIndex = INDEX_NONE;
		RegionBoundsCentroid = FVector3f(0.f);
		RegionBoundsExtents = FVector3f(0.f);
	}

	bool IsValid() const
	{
		return SourceFaceIndex != INDEX_NONE && RegionVertices.Num() > 0;
	}
};

namespace UE::MeshResizing
{
	struct FCustomRegionResizing
	{
		static UE_API void GenerateCustomRegion(const TConstArrayView<FVector3f>& BoundPositions, const UE::Geometry::FDynamicMesh3& SourceMesh, const TSet<int32>& BoundVertices, FMeshResizingCustomRegion& OutData);
		static UE_API bool CalculateFrameForCustomRegion(const UE::Geometry::FDynamicMesh3& SourceMesh, const FMeshResizingCustomRegion& BindingGroup, FVector3d& OutOrigin, FVector3f& OutTangentU, FVector3f& OutTangentV, FVector3f& OutNormal);
		static UE_API void InterpolateCustomRegionPoints(const FMeshResizingCustomRegion& BindingGroup, const TConstArrayView<FVector3d>& BoundsCorners, TArrayView<FVector3f> BoundPositions);
	};
}

#undef UE_API
