// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 
#include "UObject/NameTypes.h"
#include "Chaos/Core.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Containers/Array.h"

namespace UE::Geometry
{
	class FDynamicMesh3;
}

namespace UE::MeshResizing
{
	class FShearConstraint
	{
	public:
		MESHRESIZINGCORE_API FShearConstraint(float ShearConstraintStrength, const TArray<float>& ShearConstraintWeights, const int32 NumParticles);
		MESHRESIZINGCORE_API void Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const UE::Geometry::FDynamicMesh3& InitialResizedMesh, const UE::Geometry::FDynamicMesh3& BaseMesh) const;
		MESHRESIZINGCORE_API void Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const UE::Geometry::FDynamicMesh3& InitialResizedMesh, const UE::Geometry::FDynamicMesh3& BaseMesh, const TArray<float>& InvMass) const;

	private:
		int32 NumParticles;
		Chaos::Softs::FPBDFlatWeightMapView ShearWeightMap;
	};

	class FEdgeConstraint
	{
	public:
		MESHRESIZINGCORE_API FEdgeConstraint(float EdgeConstraintStrength, const TArray<float>& EdgeConstraintWeights, const int32 NumParticles);
		MESHRESIZINGCORE_API void Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const UE::Geometry::FDynamicMesh3& InitialResizedMesh, const UE::Geometry::FDynamicMesh3& BaseMesh, const TArray<float>& InvMass) const;

	private:
		int32 NumParticles;
		Chaos::Softs::FPBDFlatWeightMapView EdgeWeightMap;
	};

	class FBendingConstraint
	{
	public:
		MESHRESIZINGCORE_API FBendingConstraint(const UE::Geometry::FDynamicMesh3& BaseMesh, float BendingConstraintStrength, const TArray<float>& BendingConstraintWeights, const int32 NumParticles);
		Chaos::Softs::FSolverReal GetScalingFactor(const UE::Geometry::FDynamicMesh3 & Mesh, int32 ConstraintIndex, const TStaticArray<FVector3d, 4>&Grads, const Chaos::Softs::FSolverReal ExpStiffnessValue, const TArray<float>&InvMass) const;
		MESHRESIZINGCORE_API void Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const TArray<float>& InvMass) const;

	private:
		int32 NumParticles;
		Chaos::Softs::FPBDFlatWeightMapView BendingConstraintWeightMap;
		TArray<Chaos::TVec4<int32>> Constraints;
		TArray<float> RestAngles;
	};

	class FExternalForceConstraint
	{
	public:
		MESHRESIZINGCORE_API FExternalForceConstraint(const TArray<FVector3d>& InParticleExternalForce, const int32 NumParticles);
		MESHRESIZINGCORE_API void Apply(UE::Geometry::FDynamicMesh3& ResizedMesh, const TArray<float>& InvMass) const;

	private:
		int32 NumParticles;
		TArray<FVector3d> ParticleExternalForce;
	};
}

