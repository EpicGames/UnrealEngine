// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 
#include "UObject/NameTypes.h"
#include "Containers/Array.h"
#include "RBFInterpolation.generated.h"

#define UE_API MESHRESIZINGCORE_API

namespace UE::Geometry
{
	class FDynamicMesh3;
}
struct FMeshDescription;

USTRUCT()
struct FMeshResizingRBFInterpolationData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> SampleIndices;
	UPROPERTY()
	TArray<FVector3f> SampleRestPositions;
	UPROPERTY()
	TArray<float> InterpolationWeights;
};

namespace UE::MeshResizing
{
	struct FRBFInterpolation
	{
		static UE_API void GenerateWeights(const TConstArrayView<FVector3f>& SourcePositions, int32 NumInterpolationPoints, FMeshResizingRBFInterpolationData& OutData);
		static UE_API void GenerateWeights(const UE::Geometry::FDynamicMesh3& BaseMesh, int32 NumInterpolationPoints, FMeshResizingRBFInterpolationData& OutData);
		static UE_API void GenerateMeshSamples(const TArray<FVector3f>& TargetPositions, const FMeshResizingRBFInterpolationData& InterpolationData, TArray<FVector3f>& SampleDeformations);
		static UE_API void GenerateWeights(const FMeshDescription& BaseMesh, int32 NumInterpolationPoints, FMeshResizingRBFInterpolationData& OutData);
		static UE_API void DeformPoints(const TArray<FVector3f>& TargetPositions, const FMeshResizingRBFInterpolationData& InterpolationData, bool bInterpolateNormals,  UE::Geometry::FDynamicMesh3& DeformingMesh);
		static UE_API void DeformCoordinateFrames(const TArray<FVector3f>& TargetPositions, const FMeshResizingRBFInterpolationData& InterpolationData, bool bNormalize, bool bOrthogonalize, TArray<FMatrix44f>& Coordinates);
		static UE_API void DeformPoints(const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, TArray<FVector3d>& Points);
		static UE_API void DeformPoints(const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, bool bInterpolateNormals, UE::Geometry::FDynamicMesh3& DeformingMesh);
		static UE_API void DeformPoints(const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, TArrayView<FVector3f> Points, TArrayView<FVector3f> Normals);
		static UE_API void DeformPoints(const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, TArrayView<FVector3f> Points, TArrayView<FVector3f> Normals, TArrayView<FVector3f> TangentUs, TArrayView<FVector3f> TangentVs);
	};
}

#undef UE_API
