// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
public class MeshResizingEditorTools : ModuleRules
{
	public MeshResizingEditorTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"GeometryCore",
				"UnrealEd",
			}
		);
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DataflowCore",
				"DataflowEngine",
				"DataflowEditor",
				"DynamicMesh",
				"GeometryFramework",
				"InteractiveToolsFramework",
				"MeshModelingTools",
				"MeshResizingCore",
				"MeshResizingDataflowNodes",
				"ModelingComponents",
				"Slate",
				"SlateCore",
			}
			);
	}
}
