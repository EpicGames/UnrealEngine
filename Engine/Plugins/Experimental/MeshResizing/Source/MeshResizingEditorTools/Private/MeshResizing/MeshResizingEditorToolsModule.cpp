// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Dataflow/DataflowToolRegistry.h"
#include "MeshResizing/MeshResizingToolActionCommandBindings.h"
#include "MeshResizing/MeshWrapNode.h"

#define LOCTEXT_NAMESPACE "MeshResizingEditorToolsModule"

namespace UE::MeshResizing
{
	class FMeshResizingEditorTools : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
		
			const TSharedRef<const FMeshResizingToolActionCommandBindings> MeshResizingToolActions = MakeShared<FMeshResizingToolActionCommandBindings>();

			ToolRegistry.AddNodeToToolMapping(FMeshWrapLandmarksNode::StaticType(),          // Node type
				NewObject<UMeshWrapLandmarkSelectionToolBuilder>(),                          // Tool builder
				MeshResizingToolActions,                                                     // Tool action bindings
				FSlateIcon(),                                                                // Button icon 
				LOCTEXT("AddMeshWrapLandmarksNodeButtonText", "Mesh Resizing Landmarks"),    // Button text
				FName("General"),                                                            // Tool category
				FName("TObjectPtr<UDataflowMesh>"),                                          // AddNodeConnectionType
				FName("Mesh"));                                                              // AddNodeConnectionName
		}

		virtual void ShutdownModule() override
		{
			UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
			ToolRegistry.RemoveNodeToToolMapping(FMeshWrapLandmarksNode::StaticType());
		}
	};
}

IMPLEMENT_MODULE(UE::MeshResizing::FMeshResizingEditorTools, MeshResizingEditorTools)

#undef LOCTEXT_NAMESPACE
