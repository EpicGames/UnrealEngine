// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/MeshResizingToolActionCommandBindings.h"

#define LOCTEXT_NAMESPACE "MeshResizingToolActionCommandBindings"

FMeshWrapLandmarkSelectionToolActionCommands::FMeshWrapLandmarkSelectionToolActionCommands() : FMeshResizingToolActionCommands<FMeshWrapLandmarkSelectionToolActionCommands, UMeshWrapLandmarkSelectionTool>(
	TEXT("MeshWrapLandmarkSelectionToolContext"), LOCTEXT("MeshWrapLandmarkSelectionToolContext", "Mesh Wrap Landmarks Selection Tool Context"))
{}

FMeshResizingToolActionCommandBindings::FMeshResizingToolActionCommandBindings()
{
	// Note: if a TCommands<> doesn't actually register any commands then it will be deleted. Only WeightMapPaintTool currently has key commands, but we will include the other tools
	// here so that hotkeys can be added to them in the future. This means we need to check if the objects are registered before trying to use them below.

	FMeshWrapLandmarkSelectionToolActionCommands::Register();
}

void FMeshResizingToolActionCommandBindings::UnbindActiveCommands(const TSharedPtr<FUICommandList>& UICommandList) const
{
	if (FMeshWrapLandmarkSelectionToolActionCommands::IsRegistered())
	{
		FMeshWrapLandmarkSelectionToolActionCommands::Get().UnbindActiveCommands(UICommandList);
	}
}

void FMeshResizingToolActionCommandBindings::BindCommandsForCurrentTool(const TSharedPtr<FUICommandList>& UICommandList, UInteractiveTool* Tool) const
{
	if (ExactCast<UMeshWrapLandmarkSelectionTool>(Tool) && FMeshWrapLandmarkSelectionToolActionCommands::IsRegistered())
	{
		FMeshWrapLandmarkSelectionToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
	}
}

#undef LOCTEXT_NAMESPACE

