// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/MeshWrapLandmarkSelectionTool.h"
#include "Dataflow/DataflowContextObject.h"
#include "Dataflow/DataflowObject.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "Util/ColorConstants.h"
#include "ContextObjectStore.h"
#include "GroupTopology.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "PreviewMesh.h"
#include "SceneView.h"
#include "ToolContextInterfaces.h"
#include "ToolSetupUtil.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshWrapLandmarkSelectionTool)

#define LOCTEXT_NAMESPACE "MeshWrapLandmarkSelectionTool"

namespace UE::MeshResizing::Private
{
	static FLinearColor PseudoRandomColor(int32 NumColorRotations)
	{
		constexpr uint8 Spread = 157;  // Prime number that gives a good spread of colors without getting too similar as a rand might do.
		uint8 Seed = Spread;
		NumColorRotations = FMath::Abs(NumColorRotations);
		for (int32 Rotation = 0; Rotation < NumColorRotations; ++Rotation)
		{
			Seed += Spread;
		}
		return FLinearColor::MakeFromHSV8(Seed, 180, 140);
	}

	static void DrawText(FCanvas* Canvas, const FSceneView* SceneView, const FVector& Pos, const FString& Text, const FLinearColor& Color, const float Scale = 1.f)
	{
		if (Canvas && SceneView)
		{
			FVector2D PixelLocation;
			if (SceneView->WorldToPixel(Pos, PixelLocation))
			{
				// WorldToPixel doesn't account for DPIScale
				const float DPIScale = Canvas->GetDPIScale();
				FCanvasTextItem TextItem(PixelLocation / DPIScale, FText::FromString(Text), GEngine->GetSmallFont(), Color);
				TextItem.Scale = FVector2D::UnitVector * Scale;
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Draw(Canvas);
			}
		}
	}
}

// ------------------- Selection Mechanic -------------------

bool UMeshWrapLandmarkSelectionMechanic::UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut)
{
	using namespace UE::Geometry;
	if (bCtrlToggle)
	{
		// only allow selecting existing landmarks with Ctrl
		if (const UMeshWrapLandmarkSelectionTool* const SelectionTool = Cast<UMeshWrapLandmarkSelectionTool>(ParentTool))
		{
			FRay3d LocalRay(TargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
				TargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
			UE::Geometry::Normalize(LocalRay.Direction);

			const FGroupTopologySelection PreviousSelection = PersistentSelection;

			FVector3d LocalPosition, LocalNormal;
			FGroupTopologySelection Selection;
			FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings(CameraState.bIsOrthographic);
			if (TopoSelector->FindSelectedElement(TopoSelectorSettings, LocalRay, Selection, LocalPosition, LocalNormal) && Selection.SelectedCornerIDs.Num() == 1)
			{
				const int32 SelectedCornerID = Selection.GetASelectedCornerID();
				const int32 LandmarkIndex = SelectionTool->GetFirstLandmarkWithID(SelectedCornerID);
				if (LandmarkIndex != INDEX_NONE)
				{
					LocalHitPositionOut = LocalPosition;
					LocalHitNormalOut = LocalNormal;

					if (PersistentSelection != Selection || bHadCtrlOnSelection != bCtrlToggle)
					{
						bHadCtrlOnSelection = bCtrlToggle;
						PersistentSelection = Selection;
						SelectionTimestamp++;
						OnSelectionChanged.Broadcast();
						return true;
					}
				}
			}
		}
		return false;
	}
	bHadCtrlOnSelection = false;
	bHadShiftOnSelection = bShiftToggle;
	return UPolygonSelectionMechanic::UpdateSelection(WorldRay, LocalHitPositionOut, LocalHitNormalOut);
}


// ------------------- Tool -------------------
	

void UMeshWrapLandmarkSelectionTool::InitializeSculptMeshFromTarget()
{
	using namespace UE::Geometry;

	//
	// Preview 
	//

	// (Re-)Create the preview mesh
	if (PreviewMesh)
	{
		PreviewMesh->Disconnect();
	}
	PreviewMesh = NewObject<UPreviewMesh>(this, TEXT("PreviewMesh"));
	PreviewMesh->CreateInWorld(GetTargetWorld(), FTransform::Identity);

	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, Target);

	// We will use the preview mesh's spatial data structure
	PreviewMesh->bBuildSpatialDataStructure = true;

	// set materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);

	// configure secondary render material for selected triangles
	// NOTE: the material returned by ToolSetupUtil::GetSelectionMaterial has a checkerboard pattern on back faces which makes it hard to use
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial"));
	if (Material != nullptr)
	{
		if (UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, GetToolManager()))
		{
			MatInstance->SetVectorParameterValue(TEXT("Color"), FLinearColor::Yellow);
			PreviewMesh->SetSecondaryRenderMaterial(MatInstance);
		}
	}

	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	PreviewMesh->UpdatePreview(UE::ToolTarget::GetDynamicMeshCopy(Target));
	PreviewMesh->SetVisible(true);

	// Hide input target mesh
	UE::ToolTarget::HideSourceObject(Target);
}



void UMeshWrapLandmarkSelectionTool::Setup()
{
	using namespace UE::Geometry;

	InitializeSculptMeshFromTarget();

	//
	// SelectionMechanic
	//

	SelectionMechanic = NewObject<UMeshWrapLandmarkSelectionMechanic>(this);
	SelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;   // We'll do this ourselves later
	SelectionMechanic->Setup(this);
	SelectionMechanic->Properties->RestoreProperties(this);
	SelectionMechanic->Properties->bDisplayPolygroupReliantControls = false;	// this is for polygroup-specific selections like edge loops
	
	SelectionMechanic->Properties->bCanSelectVertices = true;
	SelectionMechanic->Properties->bCanSelectEdges = false;
	SelectionMechanic->Properties->bCanSelectFaces = false;
	SelectionMechanic->Properties->bSelectVertices = true;
	SelectionMechanic->Properties->bSelectEdges = false;
	SelectionMechanic->Properties->bSelectFaces = false;

	SelectionMechanic->Properties->SelectionDragTool = EMeshTopologySelectionMechanicDragTool::None;
	
	SelectionMechanic->SetShowEdges(false);
	SelectionMechanic->SetShowSelectableCorners(false);

	SelectionMechanic->PolyEdgesRenderer.DepthBias = 0.01f;
	SelectionMechanic->PolyEdgesRenderer.LineThickness = 1.0f;
	SelectionMechanic->PolyEdgesRenderer.PointSize = 2.0f;

	SelectionMechanic->SelectionRenderer.DepthBias = 0.01f;
	SelectionMechanic->SelectionRenderer.LineThickness = 1.0f;
	SelectionMechanic->SelectionRenderer.PointSize = 2.0f;

	SelectionMechanic->OnSelectionChanged.AddWeakLambda(this, [this]()
	{
		UpdateLandmarkFromSelection();
	});

	// Disallow adding/removing to the selection with shift/ctrl
	SelectionMechanic->SetShouldAddToSelectionFunc([]() { return false; });
	SelectionMechanic->SetShouldRemoveFromSelectionFunc([]() { return false; });

	// Set up Topology and SelectionMechanic using Preview's DynamicMesh
	PreviewMesh->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		Topology = MakeUnique<FTriangleGroupTopology>(&Mesh, true);

		SelectionMechanic->Initialize(&Mesh,
			FTransform3d(),
			GetTargetWorld(),
			Topology.Get(),
			[this]() { return PreviewMesh->GetSpatial(); });
	});


	//
	// Properties
	//

	ToolProperties = NewObject<UMeshWrapLandmarkSelectionToolProperties>();
	ToolProperties->WatchProperty(ToolProperties->bShowVertices, [this](bool bNewShowVertices)
	{
		SelectionMechanic->SetShowSelectableCorners(bNewShowVertices);
	});

	ToolProperties->WatchProperty(ToolProperties->bShowEdges, [this](bool bNewShowEdges)
	{
		SelectionMechanic->SetShowEdges(bNewShowEdges);
	});

	// Order of operations is important here:
	// ToolProperties->WatchProperty(ToolProperties->Name) should happen after GetSelectedNodeInfo so that we can set Landmark data.

	ToolProperties->RestoreProperties(this);
	 
	// Initialize the Selection from the selected Dataflow node
	SetPropertiesFromSelectedNode();
	if (SelectionNodeToUpdate)
	{
		SelectionNodeToUpdate->bCanDebugDraw = false; // Disable node's debug draw to not double draw
	}

	ToolProperties->WatchProperty(ToolProperties->Landmarks,
		[this](const TArray<FMeshWrapToolLandmark>& NewLandmarks)
		{
			auto SameLandmarkAsNode = [this]() -> bool
				{
					if (SelectionNodeToUpdate == nullptr || ToolProperties == nullptr)
					{
						return false;
					}
					if (ToolProperties->Landmarks.Num() != SelectionNodeToUpdate->Landmarks.Num())
					{
						return false;
					}
					const int32 NumLandmarks = ToolProperties->Landmarks.Num();
					for (int32 Idx = 0; Idx < NumLandmarks; ++Idx)
					{
						const FMeshWrapToolLandmark& ToolLandmark = ToolProperties->Landmarks[Idx];
						const FMeshWrapLandmark& NodeLandmark = SelectionNodeToUpdate->Landmarks[Idx];
						if (ToolLandmark.Identifier != NodeLandmark.Identifier)
						{
							return false;
						}
						if (ToolLandmark.VertexIndex != NodeLandmark.VertexIndex)
						{
							return false;
						}
					}
					return true;
				};
			bAnyChangeMade = !SameLandmarkAsNode();
		});
	ToolProperties->WatchProperty(ToolProperties->CurrentEditableLandmark, [this](int32 NewCurrentEditableLandmark)
		{
			SetCurrentEditableLandmark(NewCurrentEditableLandmark);
		});

	AddToolPropertySource(ToolProperties);
}

void UMeshWrapLandmarkSelectionTool::SetPropertiesFromSelectedNode()
{
	ToolProperties->Landmarks.Reset();

	SelectionNodeToUpdate = DataflowContextObject->GetSelectedNodeOfType<FMeshWrapLandmarksNode>();
	if (ensure(SelectionNodeToUpdate))
	{
		ToolProperties->Landmarks.SetNum(SelectionNodeToUpdate->Landmarks.Num());
		for (int32 Index = 0; Index < SelectionNodeToUpdate->Landmarks.Num(); ++Index)
		{
			ToolProperties->Landmarks[Index].Identifier = SelectionNodeToUpdate->Landmarks[Index].Identifier;
			ToolProperties->Landmarks[Index].VertexIndex = SelectionNodeToUpdate->Landmarks[Index].VertexIndex;

		}
	}
	ToolProperties->CurrentEditableLandmark = INDEX_NONE;
}

int32 UMeshWrapLandmarkSelectionTool::GetFirstLandmarkWithID(const int32 Id) const
{
	if (Id == INDEX_NONE)
	{
		// Special case--don't want to select a landmark when no vertex is selected.
		return INDEX_NONE;
	}
	for (int32 Index = 0; Index < ToolProperties->Landmarks.Num(); ++Index)
	{
		if (ToolProperties->Landmarks[Index].VertexIndex == Id)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

void UMeshWrapLandmarkSelectionTool::SetCurrentEditableLandmark(int32 Index)
{
	if (ToolProperties->CurrentEditableLandmark != Index)
	{
		ToolProperties->CurrentEditableLandmark = ToolProperties->Landmarks.IsValidIndex(Index) ? Index : INDEX_NONE;
		if (ToolProperties->CurrentEditableLandmark != INDEX_NONE)
		{
			// Update current selection
			FGroupTopologySelection LandmarkSelection;
			PreviewMesh->ProcessMesh([this, &LandmarkSelection](const UE::Geometry::FDynamicMesh3& Mesh)
				{
					if (Mesh.IsVertex(ToolProperties->Landmarks[ToolProperties->CurrentEditableLandmark].VertexIndex))
					{
						LandmarkSelection.SelectedCornerIDs = TSet<int32>({ ToolProperties->Landmarks[ToolProperties->CurrentEditableLandmark].VertexIndex });
					}
				});
			constexpr bool bBroadcastChange = false;
			SelectionMechanic->SetSelection(LandmarkSelection, bBroadcastChange);
		}
	}
}

void UMeshWrapLandmarkSelectionTool::UpdateLandmarkFromSelection()
{
	const UE::Geometry::FGroupTopologySelection& Selection = SelectionMechanic->GetActiveSelection();
	const int32 SelectedCornerID = Selection.GetASelectedCornerID();
	if (SelectionMechanic->GetHadCtrlOnSelection())
	{
		const int32 LandmarkIndex = GetFirstLandmarkWithID(SelectedCornerID);
		if (ToolProperties->Landmarks.IsValidIndex(LandmarkIndex))
		{
			SetCurrentEditableLandmark(LandmarkIndex);
		}
	}
	else
	{
		bool bIsValidVertex = false;
		if (SelectedCornerID != INDEX_NONE)
		{
			PreviewMesh->ProcessMesh([this, SelectedCornerID, &bIsValidVertex](const UE::Geometry::FDynamicMesh3& Mesh)
				{
					bIsValidVertex = Mesh.IsVertex(SelectedCornerID);
				});
		}
		
		if (SelectionMechanic->GetHadShiftOnSelection() || !ToolProperties->Landmarks.IsValidIndex(ToolProperties->CurrentEditableLandmark))
		{
			if (bIsValidVertex)
			{
				// Add a new landmark
				bAnyChangeMade = true;
				FMeshWrapToolLandmark& NewLandmark = ToolProperties->Landmarks.AddDefaulted_GetRef();
				NewLandmark.Identifier = TEXT("NewLandmark_") + FString::FromInt(ToolProperties->Landmarks.Num()-1);
				NewLandmark.VertexIndex = SelectedCornerID;
				ToolProperties->CurrentEditableLandmark = ToolProperties->Landmarks.Num() - 1;
			}
		}
		else
		{
			bAnyChangeMade = true;
			ToolProperties->Landmarks[ToolProperties->CurrentEditableLandmark].VertexIndex = bIsValidVertex ? SelectedCornerID : INDEX_NONE;
		}
	}
}

void UMeshWrapLandmarkSelectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept && CanAccept())
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshWrapLandmarkSelectionToolTransactionName", "Mesh Wrap Landmark"));
		UpdateSelectedNode();
		GetToolManager()->EndUndoTransaction();

		if (SelectionNodeToUpdate)
		{
			SelectionNodeToUpdate->Invalidate();
		}
	}

	SelectionMechanic->Properties->SaveProperties(this);
	ToolProperties->SaveProperties(this);

	if (PreviewMesh != nullptr)
	{
		UE::ToolTarget::ShowSourceObject(Target);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}

	SelectionMechanic->Shutdown();

	Topology.Reset();
	if (SelectionNodeToUpdate)
	{
		SelectionNodeToUpdate->bCanDebugDraw = true;
	}
}

void UMeshWrapLandmarkSelectionTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	check(PreviewMesh);
	check(RenderAPI);

	SelectionMechanic->Render(RenderAPI);

	// Display landmark points
	FPrimitiveDrawInterface* const PDI = RenderAPI->GetPrimitiveDrawInterface();
	PreviewMesh->ProcessMesh([this, PDI](const UE::Geometry::FDynamicMesh3& Mesh)
		{
			for (int32 Index = 0; Index < ToolProperties->Landmarks.Num(); ++Index)
			{
				if (Mesh.IsVertex(ToolProperties->Landmarks[Index].VertexIndex))
				{
					if (Index != ToolProperties->CurrentEditableLandmark)
					{
						PDI->DrawPoint(Mesh.GetVertexRef(ToolProperties->Landmarks[Index].VertexIndex), UE::MeshResizing::Private::PseudoRandomColor(Index), 10.f, SDPG_World);
					}
				}
			}
		});
}

void UMeshWrapLandmarkSelectionTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	check(RenderAPI);
	check(Canvas);

	SelectionMechanic->DrawHUD(Canvas, RenderAPI);

	static const FLinearColor SelectionColor = UE::Geometry::LinearColors::Gold3f();
	// Display landmark names
	if (const FSceneView* const SceneView = RenderAPI->GetSceneView())
	{
		PreviewMesh->ProcessMesh([this, Canvas, SceneView](const UE::Geometry::FDynamicMesh3& Mesh)
			{
				for (int32 Index = 0; Index < ToolProperties->Landmarks.Num(); ++Index)
				{
					if (Mesh.IsVertex(ToolProperties->Landmarks[Index].VertexIndex))
					{
						UE::MeshResizing::Private::DrawText(Canvas, SceneView, Mesh.GetVertexRef(ToolProperties->Landmarks[Index].VertexIndex), FString::FromInt(Index) + FString(" ") + ToolProperties->Landmarks[Index].Identifier,
							Index == ToolProperties->CurrentEditableLandmark ? SelectionColor : UE::MeshResizing::Private::PseudoRandomColor(Index));
					}
				}
			});
	}
}

bool UMeshWrapLandmarkSelectionTool::CanAccept() const
{
	return bAnyChangeMade;
}

FBox UMeshWrapLandmarkSelectionTool::GetWorldSpaceFocusBox()
{
	static constexpr bool bWorld = true;
	return FBox(SelectionMechanic->GetSelectionBounds(bWorld));
}

void UMeshWrapLandmarkSelectionTool::SetDataflowContextObject(TObjectPtr<UDataflowContextObject> InDataflowContextObject)
{
	DataflowContextObject = InDataflowContextObject;
}

void UMeshWrapLandmarkSelectionTool::UpdateSelectedNode()
{
	if (ensureMsgf(SelectionNodeToUpdate, TEXT("Expected non-null pointer to Selection Node")))
	{
		// Save previous state for undo
		if (UDataflow* const Dataflow = DataflowContextObject->GetDataflowAsset())
		{
			GetToolManager()->GetContextTransactionsAPI()->AppendChange(Dataflow,
				FMeshWrapLandmarksNode::MakeSelectedNodeChange(*SelectionNodeToUpdate),
				LOCTEXT("MeshWrapLandmarkUpdate", "Update Mesh Wrap Landmarks Node"));
		}

		SelectionNodeToUpdate->Landmarks.SetNum(ToolProperties->Landmarks.Num());
		for (int32 Index = 0; Index < ToolProperties->Landmarks.Num(); ++Index)
		{
			SelectionNodeToUpdate->Landmarks[Index].Identifier = ToolProperties->Landmarks[Index].Identifier;
			SelectionNodeToUpdate->Landmarks[Index].VertexIndex = ToolProperties->Landmarks[Index].VertexIndex;
		}
	}
	
}

// ------------------- Tool Builder -------------------

void UMeshWrapLandmarkSelectionToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{
}

bool UMeshWrapLandmarkSelectionToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	{
		return DataflowContextObject->GetSelectedNodeOfType<FMeshWrapLandmarksNode>() != nullptr && SceneState.TargetManager != nullptr &&(SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1);
	}

	return false;
}

const FToolTargetTypeRequirements& UMeshWrapLandmarkSelectionToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements(UPrimitiveComponentBackedTarget::StaticClass());
	return TypeRequirements;
}



UInteractiveTool* UMeshWrapLandmarkSelectionToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UMeshWrapLandmarkSelectionTool* const NewTool = NewObject<UMeshWrapLandmarkSelectionTool>(SceneState.ToolManager);

	UToolTarget* const Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	NewTool->SetTarget(Target);
	NewTool->SetWorld(SceneState.World);

	if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	{
		NewTool->SetDataflowContextObject(DataflowContextObject);
	}

	return NewTool;
}
#undef LOCTEXT_NAMESPACE
