// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshResizing/MeshWrapNode.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "DataflowEditorTools/DataflowEditorToolBuilder.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "MeshWrapLandmarkSelectionTool.generated.h"

#define UE_API MESHRESIZINGEDITORTOOLS_API

class UDataflowContextObject;
class UPreviewMesh;

namespace UE::Geometry
{
	class FGroupTopology;
	struct FGroupTopologySelection;
}

UCLASS()
class UMeshWrapLandmarkSelectionMechanic : public UPolygonSelectionMechanic
{
	GENERATED_BODY()
public:
	virtual bool UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut) override;

	bool GetHadShiftOnSelection() const { return bHadShiftOnSelection; }
	bool GetHadCtrlOnSelection() const { return bHadCtrlOnSelection; }

private:
	bool bHadShiftOnSelection = false;
	bool bHadCtrlOnSelection = false;
};

/**
 * Mesh Wrap Landmark Selection Tool Landmark
 */
USTRUCT()
struct FMeshWrapToolLandmark
{
	GENERATED_USTRUCT_BODY()

	// Landmarks will be matched by comparing Identifiers
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap")
	FString Identifier = TEXT("");

	UPROPERTY(VisibleAnywhere, Category = "Mesh Wrap", Meta = (ClampMin = "-1"))
	int32 VertexIndex = INDEX_NONE;

	bool operator==(const FMeshWrapToolLandmark& Other) const
	{
		return Identifier == Other.Identifier && VertexIndex == Other.VertexIndex;
	}
};


UCLASS(MinimalAPI)
class UMeshWrapLandmarkSelectionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Landmark names */
	UPROPERTY(EditAnywhere, Category = Landmarks, EditFixedSize, Transient)
	TArray<FMeshWrapToolLandmark> Landmarks;

	/** Currently editable landmark. Set to -1 or hold the Shift key to add a new landmark. Hold the Ctrl key and select an existing landmark to set this value. */
	UPROPERTY(EditAnywhere, Category = Landmarks, Meta = (ClampMin = "-1"))
	int32 CurrentEditableLandmark = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category = Visualization, meta = (DisplayName = "Show Vertices"))
	bool bShowVertices = false;

	UPROPERTY(EditAnywhere, Category = Visualization, meta = (DisplayName = "Show Edges"))
	bool bShowEdges = false;

};

UCLASS(MinimalAPI)
class UMeshWrapLandmarkSelectionTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	int32 GetFirstLandmarkWithID(const int32 Id) const;

private:

	friend class UMeshWrapLandmarkSelectionToolBuilder;

	virtual void Setup() override;
	virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override;

	// IInteractiveToolCameraFocusAPI implementation
	virtual FBox GetWorldSpaceFocusBox() override;

	void SetDataflowContextObject(TObjectPtr<UDataflowContextObject> InDataflowContextObject);

	void UpdateSelectedNode();
	void InitializeSculptMeshFromTarget();
	void SetPropertiesFromSelectedNode();
	void SetCurrentEditableLandmark(int32 Index);
	void UpdateLandmarkFromSelection();

	UPROPERTY()
	TObjectPtr<UMeshWrapLandmarkSelectionToolProperties> ToolProperties;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshWrapLandmarkSelectionMechanic> SelectionMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UDataflowContextObject> DataflowContextObject = nullptr;

	TUniquePtr<UE::Geometry::FGroupTopology> Topology;

	bool bAnyChangeMade = false;

	FMeshWrapLandmarksNode* SelectionNodeToUpdate = nullptr;

	FToolDataVisualizer LandmarkVisualizer;

};


UCLASS(MinimalAPI)
class UMeshWrapLandmarkSelectionToolBuilder : public UInteractiveToolWithToolTargetsBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()

private:

	// IDataflowEditorToolBuilder
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;

	// UInteractiveToolWithToolTargetsBuilder
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

#undef UE_API
