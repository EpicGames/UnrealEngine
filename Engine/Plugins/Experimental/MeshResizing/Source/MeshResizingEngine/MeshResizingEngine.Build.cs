// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MeshResizingEngine : ModuleRules
	{
		public MeshResizingEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			bTreatAsEngineModule = true;
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				}
			);
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{	
				}
			);
		}
	}
}
