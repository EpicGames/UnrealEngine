// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
public class MeshResizingDataflowNodes : ModuleRules
{
	public MeshResizingDataflowNodes(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"GeometryCore",
			}
		);
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DataflowCore",
				"DataflowEngine",
				"DataflowNodes",
				"DynamicMesh",
				"GeometryFramework",
				"MeshConversion",
				"MeshConversionEngineTypes",
				"MeshResizingCore",
				"DynamicMesh",
				"ImageCore"
			}
			);

		if(Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DataflowEditor",
				}
				);
		}
	}
}
