// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshResizing/AlignUVMeshNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowMesh.h"
#include "UDynamicMesh.h"
#include "Math/TransformCalculus2D.h"
#include "Selections/MeshConnectedComponents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AlignUVMeshNode)

using UE::Geometry::FDynamicMesh3;

namespace UE::MeshResizing
{
	void RegisterAlignUVMeshNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAlignUVMeshNode);
	}

	namespace Private
	{
		// Attempt to find a single rotation, translation and uniform scale that best transforms Source to Dest (AKA Procrustes/Procrustean transform)
		FTransform2f BestFitTransform(const TArray<FVector2f>& Source, const TArray<FVector2f>& Dest, bool bAllowScale)
		{	
			// minimize_T |S*T - D|^2,  where S = Source, D = Dest, T is a (rotation, scale, translation) linear transform
			// Solve A^T * A * x = A^T * b,   where x = {s*cos(theta), s*sin(theta), tx, ty}

			FMatrix44d ATA(EForceInit::ForceInitToZero);		// A^T * A
			for (const FVector2f& S : Source)
			{
				ATA.M[0][0] += S[0] * S[0] + S[1] * S[1];
				ATA.M[0][2] += S[0];
				ATA.M[2][0] += S[0];
				ATA.M[0][3] += S[1];
				ATA.M[3][0] += S[1];
				ATA.M[1][1] += S[0] * S[0] + S[1] * S[1];
				ATA.M[1][2] += -S[1];
				ATA.M[2][1] += -S[1];
				ATA.M[1][3] += S[0];
				ATA.M[3][1] += S[0];
				ATA.M[2][2] += 1;
				ATA.M[3][3] += 1;
			}

			FVector4d ATb(0, 0, 0, 0);			// A^T * b
			for (int32 VectorIndex = 0; VectorIndex < Source.Num(); ++VectorIndex)
			{
				const FVector2f& S = Source[VectorIndex];
				const FVector2f& D = Dest[VectorIndex];

				ATb[0] += D[0] * S[0] + D[1] * S[1];
				ATb[1] += -D[0] * S[1] + D[1] * S[0];
				ATb[2] += D[0];
				ATb[3] += D[1];
			}

			const FMatrix44d ATAInv = ATA.Inverse();
			const FVector4d Solution = ATAInv.TransformFVector4(ATb);

			float CosTheta = Solution[0];		// Actually s*cos(theta) for uniform scale s
			float SinTheta = Solution[1];

			if (!bAllowScale)
			{
				// remove the scaling
				const float Scale = FMath::Sqrt(CosTheta * CosTheta + SinTheta * SinTheta);
				CosTheta = Solution[0] / Scale;
				SinTheta = Solution[1] / Scale;
			}
			
			const FVector2f Translation(Solution[2], Solution[3]);
			const FMatrix2x2f ScaleAndRotation(CosTheta, SinTheta, -SinTheta, CosTheta);
			return FTransform2f(ScaleAndRotation, Translation);
		}
	}
}

FAlignUVMeshNode::FAlignUVMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&ResizingMesh);
	RegisterInputConnection(&BaseMesh);
	RegisterInputConnection(&UVChannelIndex);
	RegisterInputConnection(&BaseUVChannelIndex);
	RegisterOutputConnection(&ResizingMesh, &ResizingMesh);
	RegisterOutputConnection(&UVChannelIndex, &UVChannelIndex);
}

void FAlignUVMeshNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	using namespace UE::MeshResizing;

	if (Out->IsA(&UVChannelIndex))
	{
		SafeForwardInput(Context, &UVChannelIndex, &UVChannelIndex);
		return;
	}
	else if (Out->IsA(&ResizingMesh))
	{
		if (TObjectPtr<UDataflowMesh> InResizingMesh = GetValue(Context, &ResizingMesh))
		{
			if (TObjectPtr<UDataflowMesh> InBaseMesh = GetValue(Context, &BaseMesh))
			{
				if (InResizingMesh->GetDynamicMesh() && InBaseMesh->GetDynamicMesh())
				{
					if (InResizingMesh->GetDynamicMeshRef().HasAttributes() && InBaseMesh->GetDynamicMeshRef().HasAttributes())
					{
						const int32 UVChannel = GetValue(Context, &UVChannelIndex);
						int32 BaseUVChannel = GetValue(Context, &BaseUVChannelIndex);
						if (BaseUVChannel == -1)
						{
							BaseUVChannel = UVChannel;
						}

						if (UVChannel >= 0 && InResizingMesh->GetDynamicMeshRef().Attributes()->NumUVLayers() > UVChannel && InBaseMesh->GetDynamicMeshRef().Attributes()->NumUVLayers() > BaseUVChannel)
						{
							const FDynamicMeshUVOverlay* const InResizedUVOverlay = InResizingMesh->GetDynamicMeshRef().Attributes()->GetUVLayer(UVChannel);

							FDynamicMesh3 ResizedMesh;
							ResizedMesh.Copy(InResizingMesh->GetDynamicMeshRef());
							FDynamicMeshUVOverlay* const ResizedUVOverlay = ResizedMesh.Attributes()->GetUVLayer(UVChannel);

							FMeshConnectedComponents UVIslands(&ResizedMesh);
							UVIslands.FindConnectedTriangles([ResizedUVOverlay](int32 Triangle0, int32 Triangle1) 
							{
								return ResizedUVOverlay ? ResizedUVOverlay->AreTrianglesConnected(Triangle0, Triangle1) : false;
							});

							const FDynamicMeshUVOverlay* const BaseUVOverlay = InBaseMesh->GetDynamicMeshRef().Attributes()->GetUVLayer(BaseUVChannel);

							for (const FMeshConnectedComponents::FComponent& Island : UVIslands.Components)
							{
								if (!ensure(Island.Indices.Num() > 0))
								{
									continue;
								}

								TSet<int32> IslandVertices;
								for (int32 TriID : Island.Indices)
								{
									FIndex3i Tri = ResizedMesh.GetTriangle(TriID);
									IslandVertices.Add(Tri[0]);
									IslandVertices.Add(Tri[1]);
									IslandVertices.Add(Tri[2]);
								}

								TArray<FVector2f> ResizedUVs, BaseUVs;
								TArray<int32> ElementIndices;
								for (int32 ElementIndex : InResizedUVOverlay->ElementIndicesItr())
								{
									const int32 VertexForElement = InResizedUVOverlay->GetParentVertex(ElementIndex);
									if (IslandVertices.Contains(VertexForElement))
									{
										ElementIndices.Add(ElementIndex);
										ResizedUVs.Add(InResizedUVOverlay->GetElement(ElementIndex));
										BaseUVs.Add(BaseUVOverlay->GetElement(ElementIndex));
									}
								}

								const FTransform2f BestFit = Private::BestFitTransform(BaseUVs, ResizedUVs, bScale);
								const FTransform2f InverseBestFit = BestFit.Inverse();

								for (int32 UVIndex = 0; UVIndex < ResizedUVs.Num(); ++UVIndex)
								{
									const int32 ElementIndex = ElementIndices[UVIndex];
									const int32 ElementVertex = InResizedUVOverlay->GetParentVertex(ElementIndex);
									if (IslandVertices.Contains(ElementVertex))
									{
										const FVector2f UV = InverseBestFit.TransformPoint(ResizedUVs[UVIndex]);
										ResizedUVOverlay->SetElement(ElementIndex, UV);
									}
								}
							}

							TObjectPtr<UDataflowMesh> OutResizedMesh = NewObject<UDataflowMesh>();
							OutResizedMesh->SetDynamicMesh(MoveTemp(ResizedMesh));
							OutResizedMesh->SetMaterials(InResizingMesh->GetMaterials());
							SetValue(Context, OutResizedMesh, &ResizingMesh);
							SetValue(Context, UVChannel, &UVChannelIndex);
							return;
						}
						else
						{
							Context.Warning(TEXT("Invalid UVChannelIndex or BaseUVChannelIndex"), this, Out);
						}
					}
					else
					{
						Context.Warning(TEXT("An input mesh does not have an AttributeSet"), this, Out);
					}
				}
				else
				{
					Context.Warning(TEXT("An input mesh does not have a DynamicMesh object"), this, Out);
				}
			}
		}
		SafeForwardInput(Context, &ResizingMesh, &ResizingMesh);
	}
}


