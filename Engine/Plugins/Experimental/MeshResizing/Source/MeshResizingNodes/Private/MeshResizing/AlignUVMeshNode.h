// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "AlignUVMeshNode.generated.h"

class UDataflowMesh;

USTRUCT(Meta = (MeshResizing, Experimental))
struct FAlignUVMeshNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAlignUVMeshNode, "AlignUVMeshNode", "MeshResizing", "Align UV Mesh")

public:

	FAlignUVMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "ResizingMesh"))
	TObjectPtr<UDataflowMesh> ResizingMesh;

	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> BaseMesh;

	UPROPERTY(EditAnywhere, Category = "Align", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "UVChannelIndex", DataflowUVChannel, ClampMin = 0, ClampMax = 7, UIMin = 0, UIMax = 7))
	int32 UVChannelIndex = 0;

	/** Base UV channel index in case it differs from the ResizingMesh UV channel index, or -1 to use the same channel. */
	UPROPERTY(EditAnywhere, Category = "Align", meta = (DataflowInput, ClampMin = -1, ClampMax = 7, UIMin = -1, UIMax = 7))
	int32 BaseUVChannelIndex = -1;

	UPROPERTY(EditAnywhere, Category = "Align")
	bool bScale = true;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};


namespace UE::MeshResizing
{
	void RegisterAlignUVMeshNodes();
}
