// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/BaseBodyDataflowNodes.h"
#include "MeshResizing/BaseBodyTools.h"
#include "Dataflow/DataflowCore.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "SkeletalMeshLODModelToDynamicMesh.h"
#include "SkeletalMeshLODRenderDataToDynamicMesh.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseBodyDataflowNodes)

namespace UE::MeshResizing
{
	void RegisterBaseBodyDataflowNodes()
	{
		static const FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkeletalMeshToMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateResizableProxyDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateInterpolatedProxyDataflowNode);

		// Deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkeletalMeshToMeshDataflowNode);

		// Auto convert
		UE_DATAFLOW_REGISTER_AUTOCONVERT(TObjectPtr<USkeletalMesh>, TObjectPtr<UDataflowMesh>, FSkeletalMeshToMeshDataflowNode_v2);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

void FSkeletalMeshToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::MeshResizing;
	if (Out->IsA(&Mesh))
	{
		if (const USkeletalMesh* const InSkeletalMesh = GetValue(Context, &SkeletalMesh))
		{
#if WITH_EDITORONLY_DATA
			if (bRecordImportedVertices)
			{
				if (const FSkeletalMeshModel* const MeshModel = InSkeletalMesh->GetImportedModel())
				{
					if (MeshModel->LODModels.IsValidIndex(LODLevel))
					{
						const FSkeletalMeshLODModel* const SkeletalMeshLODModel = &MeshModel->LODModels[LODLevel];

						TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
						NewMesh->Reset();

						UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

						FSkeletalMeshLODModelToDynamicMesh Converter;
						constexpr bool bCopyTangentsTrue = true;
						Converter.Convert(SkeletalMeshLODModel, DynMesh, bCopyTangentsTrue); // Tangents are needed by RigidBinding

						// Set ImportedVertices
						TArray<int32> VertexToImportedVertexIDMap;
						VertexToImportedVertexIDMap.SetNumUninitialized(DynMesh.VertexCount());
						TArray<int32> VertexToRawPointIndicesMap;
						VertexToRawPointIndicesMap.SetNumUninitialized(DynMesh.VertexCount());
						check(DynMesh.VertexCount() == Converter.VertIDMap.Num());

						const TArray<int32>& LODModelToImportVertexMap = SkeletalMeshLODModel->MeshToImportVertexMap;
						const TArray<uint32>& RawPointIndices = SkeletalMeshLODModel->GetRawPointIndices();

						for (int32 DynMeshVertexIdx = 0; DynMeshVertexIdx < DynMesh.VertexCount(); ++DynMeshVertexIdx)
						{
							const int32 LODModelVertexIdx = Converter.VertIDMap[DynMeshVertexIdx];
							const int32 ImportVertexIdx = LODModelToImportVertexMap[LODModelVertexIdx];
							const int32 RawPointIdx = RawPointIndices[LODModelVertexIdx];
							VertexToImportedVertexIDMap[DynMeshVertexIdx] = ImportVertexIdx;
							VertexToRawPointIndicesMap[DynMeshVertexIdx] = RawPointIdx;
						}
						FBaseBodyTools::AttachVertexMappingData(FBaseBodyTools::ImportedVertexVIDsAttrName, VertexToImportedVertexIDMap, DynMesh);
						FBaseBodyTools::AttachVertexMappingData(FBaseBodyTools::RawPointIndicesVIDsAttrName, VertexToRawPointIndicesMap, DynMesh);

						SetValue(Context, NewMesh, &Mesh);
						return;
					}
				}
			}
			else if (bUseMeshDescription)
			{
				if (const FMeshDescription* const MeshDescription = InSkeletalMesh->GetMeshDescription(LODLevel))
				{
					TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
					NewMesh->Reset();

					UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();
					FMeshDescriptionToDynamicMesh Converter;
					Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
					Converter.Convert(MeshDescription, DynMesh, true);

					SetValue(Context, NewMesh, &Mesh);
					return;
				}
			}
			else
#endif
			{
				if (const FSkeletalMeshRenderData* const RenderData = InSkeletalMesh->GetResourceForRendering())
				{
					if (RenderData->LODRenderData.IsValidIndex(LODLevel))
					{
						const FSkeletalMeshLODRenderData* const SkeletalMeshLODRenderData = &RenderData->LODRenderData[LODLevel];

						TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
						NewMesh->Reset();

						UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

						UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::ConversionOptions  ConversionOptions;
						UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::Convert(SkeletalMeshLODRenderData, InSkeletalMesh->GetRefSkeleton(), ConversionOptions, DynMesh);

						SetValue(Context, NewMesh, &Mesh);
						return;
					}
				}
			}
		}
		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
	}
	else if (Out->IsA(&MaterialArray))
	{
		TArray<TObjectPtr<UMaterialInterface>> OutMaterials;
		if (const USkeletalMesh* const InSkeletalMesh = GetValue(Context, &SkeletalMesh))
		{
			const TArray<FSkeletalMaterial>& InMaterials = InSkeletalMesh->GetMaterials();
			OutMaterials.Reserve(InMaterials.Num());
			for (const FSkeletalMaterial& Material : InMaterials)
			{
				OutMaterials.Emplace(Material.MaterialInterface);
			}
		}
		SetValue(Context, OutMaterials, &MaterialArray);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

FSkeletalMeshToMeshDataflowNode_v2::FSkeletalMeshToMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&SkeletalMesh);
	RegisterOutputConnection(&Mesh);
}

void FSkeletalMeshToMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::MeshResizing;
	if (Out->IsA(&Mesh))
	{
		if (const USkeletalMesh* const InSkeletalMesh = GetValue(Context, &SkeletalMesh))
		{
			// Materials
			TArray<TObjectPtr<UMaterialInterface>> OutMaterials;

			const TArray<FSkeletalMaterial>& InMaterials = InSkeletalMesh->GetMaterials();
			OutMaterials.Reserve(InMaterials.Num());
			for (const FSkeletalMaterial& Material : InMaterials)
			{
				OutMaterials.Emplace(Material.MaterialInterface);
			}

#if WITH_EDITORONLY_DATA
			if (bRecordImportedVertices)
			{
				if (const FSkeletalMeshModel* const MeshModel = InSkeletalMesh->GetImportedModel())
				{
					if (MeshModel->LODModels.IsValidIndex(LODLevel))
					{
						const FSkeletalMeshLODModel* const SkeletalMeshLODModel = &MeshModel->LODModels[LODLevel];

						TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
						NewMesh->Reset();

						UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

						FSkeletalMeshLODModelToDynamicMesh Converter;
						constexpr bool bCopyTangentsTrue = true;
						Converter.Convert(SkeletalMeshLODModel, DynMesh, bCopyTangentsTrue); // Tangents are needed by RigidBinding

						// Set ImportedVertices
						TArray<int32> VertexToImportedVertexIDMap;
						VertexToImportedVertexIDMap.SetNumUninitialized(DynMesh.VertexCount());
						TArray<int32> VertexToRawPointIndicesMap;
						VertexToRawPointIndicesMap.SetNumUninitialized(DynMesh.VertexCount());
						check(DynMesh.VertexCount() == Converter.VertIDMap.Num());

						const TArray<int32>& LODModelToImportVertexMap = SkeletalMeshLODModel->MeshToImportVertexMap;
						const TArray<uint32>& RawPointIndices = SkeletalMeshLODModel->GetRawPointIndices();
						
						for (int32 DynMeshVertexIdx = 0; DynMeshVertexIdx < DynMesh.VertexCount(); ++DynMeshVertexIdx)
						{
							const int32 LODModelVertexIdx = Converter.VertIDMap[DynMeshVertexIdx];
							const int32 ImportVertexIdx = LODModelToImportVertexMap[LODModelVertexIdx];
							const int32 RawPointIdx = RawPointIndices[LODModelVertexIdx];
							VertexToImportedVertexIDMap[DynMeshVertexIdx] = ImportVertexIdx;
							VertexToRawPointIndicesMap[DynMeshVertexIdx] = RawPointIdx;
						}
						FBaseBodyTools::AttachVertexMappingData(FBaseBodyTools::ImportedVertexVIDsAttrName, VertexToImportedVertexIDMap, DynMesh);
						FBaseBodyTools::AttachVertexMappingData(FBaseBodyTools::RawPointIndicesVIDsAttrName, VertexToRawPointIndicesMap, DynMesh);

						TObjectPtr<UDataflowMesh> NewDataflowMesh = NewObject<UDataflowMesh>();
						NewDataflowMesh->SetDynamicMesh(DynMesh);
						NewDataflowMesh->SetMaterials(OutMaterials);

						SetValue(Context, NewDataflowMesh, &Mesh);
						return;
					}
				}
			}
			else if (bUseMeshDescription)
			{
				if (const FMeshDescription* const MeshDescription = InSkeletalMesh->GetMeshDescription(LODLevel))
				{
					TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
					NewMesh->Reset();

					UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();
					FMeshDescriptionToDynamicMesh Converter;
					Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
					Converter.Convert(MeshDescription, DynMesh, true);

					TObjectPtr<UDataflowMesh> NewDataflowMesh = NewObject<UDataflowMesh>();
					NewDataflowMesh->SetDynamicMesh(DynMesh);
					NewDataflowMesh->SetMaterials(OutMaterials);

					SetValue(Context, NewDataflowMesh, &Mesh);
					return;
				}
			}
			else
#endif
			{
				if (const FSkeletalMeshRenderData* const RenderData = InSkeletalMesh->GetResourceForRendering())
				{
					if (RenderData->LODRenderData.IsValidIndex(LODLevel))
					{
						const FSkeletalMeshLODRenderData* const SkeletalMeshLODRenderData = &RenderData->LODRenderData[LODLevel];

						TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
						NewMesh->Reset();

						UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

						UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::ConversionOptions  ConversionOptions;
						UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::Convert(SkeletalMeshLODRenderData, InSkeletalMesh->GetRefSkeleton(), ConversionOptions, DynMesh);

						TObjectPtr<UDataflowMesh> NewDataflowMesh = NewObject<UDataflowMesh>();
						NewDataflowMesh->SetDynamicMesh(DynMesh);
						NewDataflowMesh->SetMaterials(OutMaterials);

						SetValue(Context, NewDataflowMesh, &Mesh);
						return;
					}
				}
			}
		}

		SetValue(Context, TObjectPtr<UDataflowMesh>(NewObject<UDataflowMesh>()), &Mesh);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

void FGenerateResizableProxyDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TargetProxyMesh))
	{
		if (TObjectPtr<UDynamicMesh> InSourceMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &SourceMesh))
		{
			if (TObjectPtr<UDynamicMesh> InTargetMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &TargetMesh))
			{
				const UE::Geometry::FDynamicMesh3& SourceDynMesh = InSourceMesh->GetMeshRef();
				const UE::Geometry::FDynamicMesh3& TargetDynMesh = InTargetMesh->GetMeshRef();

				TObjectPtr<UDynamicMesh> NewTargetMesh = NewObject<UDynamicMesh>();
				NewTargetMesh->Reset();
				UE::Geometry::FDynamicMesh3& NewTargetDynMesh = NewTargetMesh->GetMeshRef();

				if (UE::MeshResizing::FBaseBodyTools::GenerateResizableProxyFromVertexMappingData(SourceDynMesh, FName(*SourceMappingData), TargetDynMesh, FName(*TargetMappingData), NewTargetDynMesh))
				{
					SetValue(Context, NewTargetMesh, &TargetProxyMesh);
					return;
				}
			}
		}
		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &TargetProxyMesh);
	}
	else if (Out->IsA(&SourceProxyMesh))
	{
		SafeForwardInput(Context, &SourceMesh, &SourceProxyMesh);
	}
	else if (Out->IsA(&ProxyMaterialArray))
	{
		SafeForwardInput(Context, &SourceMaterialArray, &ProxyMaterialArray);
	}
}

void FGenerateInterpolatedProxyDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ProxyMesh))
	{
		if (BlendAlpha == 0.f)
		{
			SafeForwardInput(Context, &SourceMesh, &ProxyMesh);
		}
		else if (BlendAlpha == 1.f)
		{
			SafeForwardInput(Context, &TargetMesh, &ProxyMesh);
		}
		else
		{
			if (TObjectPtr<UDynamicMesh> InSourceMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &SourceMesh))
			{
				if (TObjectPtr<UDynamicMesh> InTargetMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &TargetMesh))
				{
					const UE::Geometry::FDynamicMesh3& SourceDynMesh = InSourceMesh->GetMeshRef();
					const UE::Geometry::FDynamicMesh3& TargetDynMesh = InTargetMesh->GetMeshRef();

					TObjectPtr<UDynamicMesh> NewTargetMesh = NewObject<UDynamicMesh>();
					NewTargetMesh->Reset();
					UE::Geometry::FDynamicMesh3& NewTargetDynMesh = NewTargetMesh->GetMeshRef();

					if (UE::MeshResizing::FBaseBodyTools::InterpolateResizableProxy(SourceDynMesh, TargetDynMesh, BlendAlpha, NewTargetDynMesh))
					{
						SetValue(Context, NewTargetMesh, &ProxyMesh);
						return;
					}
				}
			}
			SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &ProxyMesh);
		}
	}
	else if (Out->IsA(&ProxyMaterialArray))
	{
		SafeForwardInput(Context, &SourceMaterialArray, &ProxyMaterialArray);
	}
}