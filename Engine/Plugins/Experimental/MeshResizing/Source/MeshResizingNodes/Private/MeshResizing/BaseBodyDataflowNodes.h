// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowEngine.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshProcessor.h"
#include "Dataflow/DataflowMesh.h"

#include "BaseBodyDataflowNodes.generated.h"

class USkeletalMesh;
class UDynamicMesh;
class UMaterialInterface;

/**
 *
 * Converts a SkeletalMesh into a DynamicMesh with Imported Vertex information
 * DEPRECATED 5.8 - use FSkeletalMeshToMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (MeshResizing, Experimental, Deprecated = "5.8"))
struct FSkeletalMeshToMeshDataflowNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkeletalMeshToMeshDataflowNode, "SkeletalMeshToMesh", "Mesh|Utilities", "")

public:

	/** SkeletalMesh to convert */
	UPROPERTY(EditAnywhere, Category = "SkeletalMesh", meta = (DataflowInput));
	TObjectPtr<const USkeletalMesh> SkeletalMesh;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Output materials */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	FSkeletalMeshToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SkeletalMesh);
		RegisterOutputConnection(&Mesh);
		RegisterOutputConnection(&MaterialArray);
	}

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** Specifies the LOD level to use */
	UPROPERTY(EditAnywhere, Category = "SkeletalMesh", meta = (DisplayName = "LOD Level"));
	int32 LODLevel = 0;

#if WITH_EDITORONLY_DATA

	/** Generate from the SkeletalMeshLODModel (vertex order will match SKM vertex order). Record ImportedVertices (if available) as NonManifold mapping data. This requires Editor-Only data.*/
	UPROPERTY(EditAnywhere, Category = "SkeletalMesh")
	bool bRecordImportedVertices = true;

	/** Generate from mesh description (vertex order will match mesh description / ImportedVertices). Requires Editor-Only data.*/
	UPROPERTY(EditAnywhere, Category = "SkeletalMesh", meta = (EditCondition = "!bRecordImportedVertices"))
	bool bUseMeshDescription = false;
#endif
};


/**
 *
 * Generate a pair of Dynamic Meshes with the same topology that can be interpolated.
 * 
 * Currently, this node relies on the vertex mapping data existing on the input source and target meshes,
 * and that the mapped vertices on both meshes match.
 */
USTRUCT(meta = (MeshResizing, Experimental))
struct FGenerateResizableProxyDataflowNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateResizableProxyDataflowNode, "GenerateResizableProxy", "MeshResizing", "")

public:

	/** Source mesh */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDynamicMesh> SourceMesh;

	/** Source materials.*/
	UPROPERTY(meta = (DataflowInput))
	TArray<TObjectPtr<UMaterialInterface>> SourceMaterialArray;

	/** Target mesh */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDynamicMesh> TargetMesh;

	/** Output source proxy mesh */
	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "SourceMesh"))
	TObjectPtr<UDynamicMesh> SourceProxyMesh;

	/** Output source proxy mesh */
	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "SourceMesh"))
	TObjectPtr<UDynamicMesh> TargetProxyMesh;

	/** Target materials.*/
	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "SourceMaterialArray"))
	TArray<TObjectPtr<UMaterialInterface>> ProxyMaterialArray;


	FGenerateResizableProxyDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SourceMesh);
		RegisterInputConnection(&SourceMaterialArray);
		RegisterInputConnection(&TargetMesh);
		RegisterOutputConnection(&SourceProxyMesh, &SourceMesh);
		RegisterOutputConnection(&TargetProxyMesh, &SourceMesh);
		RegisterOutputConnection(&ProxyMaterialArray, &SourceMaterialArray);
	}

private:

	/** Source vertex mapping data. TODO: only have two choices that work currently. Make this an enum or something */
	UPROPERTY(EditAnywhere, Category = "Resizable Proxy")
	FString SourceMappingData = "ImportedVertexVIDsAttr";

	/** Target vertex mapping data. TODO: only have two choices that work currently. Make this an enum or something */
	UPROPERTY(EditAnywhere, Category = "Resizable Proxy")
	FString TargetMappingData = "ImportedVertexVIDsAttr";

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
/**
 *
 * Generate a pair of Dynamic Meshes with the same topology that can be interpolated.
 *
 * Currently, this node relies on the vertex mapping data existing on the input source and target meshes,
 * and that the mapped vertices on both meshes match.
 */
USTRUCT(meta = (MeshResizing, Experimental))
struct FGenerateInterpolatedProxyDataflowNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateInterpolatedProxyDataflowNode, "GenerateInterpolatedProxy", "MeshResizing", "")

public:

	/** Source mesh */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDynamicMesh> SourceMesh;

	/** Source materials.*/
	UPROPERTY(meta = (DataflowInput))
	TArray<TObjectPtr<UMaterialInterface>> SourceMaterialArray;

	/** Target mesh */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDynamicMesh> TargetMesh;

	/** Output proxy mesh */
	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "SourceMesh"))
	TObjectPtr<UDynamicMesh> ProxyMesh;

	/** Proxy materials.*/
	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "SourceMaterialArray"))
	TArray<TObjectPtr<UMaterialInterface>> ProxyMaterialArray;


	FGenerateInterpolatedProxyDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SourceMesh);
		RegisterInputConnection(&SourceMaterialArray);
		RegisterInputConnection(&TargetMesh);
		RegisterOutputConnection(&ProxyMesh, &SourceMesh);
		RegisterOutputConnection(&ProxyMaterialArray, &SourceMaterialArray);
	}

private:

	/** Alpha between source (0) and target (1) */
	UPROPERTY(EditAnywhere, Category = "Interpolate Proxy", meta = (ClampMin = "0", ClampMax = "1"))
	float BlendAlpha = 1.f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Converts a SkeletalMesh into a DataflowMesh with Imported Vertex information
 *
 */
USTRUCT(meta = (MeshResizing, Experimental, Icon = "Icons.SkeletalMesh"))
struct FSkeletalMeshToMeshDataflowNode_v2 final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkeletalMeshToMeshDataflowNode_v2, "SkeletalMeshToMesh", "Mesh|Utilities", "")

private:

	/** SkeletalMesh to convert */
	UPROPERTY(EditAnywhere, Category = "SkeletalMesh", meta = (DataflowInput));
	TObjectPtr<const USkeletalMesh> SkeletalMesh;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** Specifies the LOD level to use */
	UPROPERTY(EditAnywhere, Category = "SkeletalMesh", meta = (DisplayName = "LOD Level"));
	int32 LODLevel = 0;

#if WITH_EDITORONLY_DATA
	/** Generate from the SkeletalMeshLODModel (vertex order will match SKM vertex order). Record ImportedVertices (if available) as NonManifold mapping data. This requires Editor-Only data.*/
	UPROPERTY(EditAnywhere, Category = "SkeletalMesh")
	bool bRecordImportedVertices = true;

	/** Generate from mesh description (vertex order will match mesh description / ImportedVertices). Requires Editor-Only data.*/
	UPROPERTY(EditAnywhere, Category = "SkeletalMesh", meta = (EditCondition = "!bRecordImportedVertices"))
	bool bUseMeshDescription = false;
#endif

public:
	FSkeletalMeshToMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::MeshResizing
{
	void RegisterBaseBodyDataflowNodes();
}

