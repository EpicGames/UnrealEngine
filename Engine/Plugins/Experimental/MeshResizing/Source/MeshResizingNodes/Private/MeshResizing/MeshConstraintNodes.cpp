// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshResizing/MeshConstraintNodes.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowMesh.h"
#include "MeshResizing/Mesh3DConstraints.h"
#include "UDynamicMesh.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshConstraintNodes)

using UE::Geometry::FDynamicMesh3;

namespace  UE::MeshResizing
{
	void RegisterMeshConstraintDataflowNodes()
	{
		static const FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshConstrainedDeformationNode);
	}

	namespace Private
	{
		typedef TFunction<void (UE::Geometry::FDynamicMesh3& ResizedMesh, const UE::Geometry::FDynamicMesh3& InitialResizedMesh, const UE::Geometry::FDynamicMesh3& BaseMesh, int32 Iteration)> ApplyConstraintFunc;

		static void Solve(UE::Geometry::FDynamicMesh3& ResizedMesh, const UE::Geometry::FDynamicMesh3& InitialResizedMesh, const UE::Geometry::FDynamicMesh3& BaseMesh, int32 Iterations,
			const TArray<ApplyConstraintFunc>& Constraints)
		{
			if (Constraints.IsEmpty())
			{
				return;
			}
			for (int32 Iteration = 0; Iteration < Iterations; ++Iteration)
			{
				for (const ApplyConstraintFunc& Constraint : Constraints)
				{
					Constraint(ResizedMesh, InitialResizedMesh, BaseMesh, Iteration);
				}
			}
		}
	}
}

FMeshConstrainedDeformationNode::FMeshConstrainedDeformationNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&ResizingMesh);
	RegisterInputConnection(&BaseMesh);
	RegisterInputConnection(&InvMass);
	RegisterInputConnection(&EdgeConstraintWeights);
	RegisterOutputConnection(&ResizingMesh, &ResizingMesh);
}

void FMeshConstrainedDeformationNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	
	using namespace UE::Geometry;
	using namespace UE::MeshResizing;
	using namespace Chaos::Softs;

	if (Out->IsA(&ResizingMesh))
	{
		if (TObjectPtr<UDataflowMesh> InResizingMesh = GetValue(Context, &ResizingMesh))
		{
			if (TObjectPtr<UDataflowMesh> InBaseMesh = GetValue(Context, &BaseMesh))
			{
				if (InResizingMesh->GetDynamicMesh() && InBaseMesh->GetDynamicMesh())
				{
					FDynamicMesh3 ResizedMesh;
					ResizedMesh.Copy(InResizingMesh->GetDynamicMeshRef());

					TArray<Private::ApplyConstraintFunc> Constraints;
					
					TArray<FVector3d> ExternalForce;
					ExternalForce.Init(Gravity, ResizedMesh.MaxVertexID());
					UE::MeshResizing::FExternalForceConstraint ExternalForceConstraint(ExternalForce, ResizedMesh.MaxVertexID());

					TArray<float> ShearConstraintWeights;
					ShearConstraintWeights.Init(1.f, ResizedMesh.MaxVertexID());
					const TArray<float> ParticleInvMass = GetValue(Context, &InvMass);
					UE::MeshResizing::FShearConstraint ShearConstraint(ShearConstraintStrength, ShearConstraintWeights, ResizedMesh.MaxVertexID());
					UE::MeshResizing::FEdgeConstraint EdgeConstraint(EdgeConstraintStrength, GetValue(Context, &EdgeConstraintWeights), ResizedMesh.MaxVertexID());
					
					TArray<float> BendingConstraintWeights;
					BendingConstraintWeights.Init(1.f, ResizedMesh.MaxVertexID());
					UE::MeshResizing::FBendingConstraint BendingConstraint(InBaseMesh->GetDynamicMeshRef(), BendingConstraintStrength, BendingConstraintWeights, ResizedMesh.MaxVertexID());

					// Pre-update: external forces
					Constraints.Add(
						[&ExternalForceConstraint, &ParticleInvMass](UE::Geometry::FDynamicMesh3& InResizedMesh, const UE::Geometry::FDynamicMesh3& InInitialResizedMesh, const UE::Geometry::FDynamicMesh3& InBaseMesh, int32 Iteration)
						{
							ExternalForceConstraint.Apply(InResizedMesh, ParticleInvMass);
						}
					);
					if (bEnableShearConstraint)
					{
						Constraints.Add(
							[&ShearConstraint, &ParticleInvMass](UE::Geometry::FDynamicMesh3& InResizedMesh, const UE::Geometry::FDynamicMesh3& InInitialResizedMesh, const UE::Geometry::FDynamicMesh3& InBaseMesh, int32 Iteration)
							{
								ShearConstraint.Apply(InResizedMesh, InInitialResizedMesh, InBaseMesh, ParticleInvMass);
							}
						);
					}
					
					if (bEnableBendingConstraint)
					{
						Constraints.Add(
							[&BendingConstraint, &ParticleInvMass](UE::Geometry::FDynamicMesh3& InResizedMesh, const UE::Geometry::FDynamicMesh3& InInitialResizedMesh, const UE::Geometry::FDynamicMesh3& InBaseMesh, int32 Iteration)
							{
								BendingConstraint.Apply(InResizedMesh, ParticleInvMass);
							}
						);
					}

					if (bEnableEdgeConstraint)
					{
						Constraints.Add(
							[&EdgeConstraint, &ParticleInvMass](UE::Geometry::FDynamicMesh3& InResizedMesh, const UE::Geometry::FDynamicMesh3& InInitialResizedMesh, const UE::Geometry::FDynamicMesh3& InBaseMesh, int32 Iteration)
							{
								EdgeConstraint.Apply(InResizedMesh, InInitialResizedMesh, InBaseMesh, ParticleInvMass);
							}
						);
					}

					UE::MeshResizing::Private::Solve(ResizedMesh, InResizingMesh->GetDynamicMeshRef(), InBaseMesh->GetDynamicMeshRef(), Iterations, Constraints);

					TObjectPtr<UDataflowMesh> OutResizedMesh = NewObject<UDataflowMesh>();
					OutResizedMesh->SetDynamicMesh(MoveTemp(ResizedMesh));
					OutResizedMesh->SetMaterials(InResizingMesh->GetMaterials());
					SetValue(Context, OutResizedMesh, &ResizingMesh);
					return;
				}
			}
		}
		SafeForwardInput(Context, &ResizingMesh, &ResizingMesh);
	}	
}