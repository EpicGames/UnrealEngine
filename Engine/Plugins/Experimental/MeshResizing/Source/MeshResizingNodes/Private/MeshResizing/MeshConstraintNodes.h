// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "MeshConstraintNodes.generated.h"

class UDataflowMesh;
class UMaterialInterface;


USTRUCT(Meta = (MeshResizing, Experimental))
struct FMeshConstrainedDeformationNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshConstrainedDeformationNode, "MeshConstrainedDeformationTestPlayground", "MeshResizing", "Mesh Constrained Deformation")

public:

	FMeshConstrainedDeformationNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "ResizingMesh"))
	TObjectPtr<UDataflowMesh> ResizingMesh;

	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> BaseMesh;

	UPROPERTY(meta = (DataflowInput))
	TArray<float> InvMass;

	UPROPERTY(meta = (DataflowInput))
	TArray<float> EdgeConstraintWeights;

	UPROPERTY(EditAnywhere, Category = "Constrained Deformation|Solver", meta = (Min = 0, UIMax = 100))
	int32 Iterations = 100;

	/** Remove shear deformation */
	UPROPERTY(EditAnywhere, Category = "Constrained Deformation|Shear Constraint")
	bool bEnableShearConstraint = true;
	UPROPERTY(EditAnywhere, Category = "Constrained Deformation|Shear Constraint", meta = (Min = 0, Max = 1, EditCondition = "bEnableShearConstraint"))
	float ShearConstraintStrength = 1.f;

	UPROPERTY(EditAnywhere, Category = "Constrained Deformation|Bending Constraints")
	bool bEnableBendingConstraint = true;
	UPROPERTY(EditAnywhere, Category = "Constrained Deformation|Bending Constraints", meta = (Min = 0, EditCondition = "bEnableBendingConstraint"))
	float BendingConstraintStrength = 1.f;

	UPROPERTY(EditAnywhere, Category = "Constrained Deformation|Edge Constraints")
	bool bEnableEdgeConstraint = true;
	UPROPERTY(EditAnywhere, Category = "Constrained Deformation|Edge Constraints", meta = (Min = 0, EditCondition = "bEnableEdgeConstraint"))
	float EdgeConstraintStrength = 1.f;

	UPROPERTY(EditAnywhere, Category = "Gravity")
	FVector3d Gravity = FVector3d(0,0,0);
	
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

namespace UE::MeshResizing
{
	void RegisterMeshConstraintDataflowNodes();
}