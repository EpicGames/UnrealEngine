// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Modules/ModuleManager.h"
#include "MeshResizing/AlignUVMeshNode.h"
#include "MeshResizing/BaseBodyDataflowNodes.h"
#include "MeshResizing/MeshConstraintNodes.h"
#include "MeshResizing/MeshResizingTextureNodes.h"
#include "MeshResizing/MeshWarpNode.h"
#include "MeshResizing/MeshWrapNode.h"
#include "MeshResizing/RBFInterpolationNodes.h"
#include "MeshResizing/UVUnwrapNode.h"
#include "MeshResizing/UVMeshTransformNode.h"
#include "MeshResizing/UVResizeControllerNode.h"

namespace UE::MeshResizing
{
	namespace Private
	{
		struct FColorScheme
		{
			static inline const FLinearColor Asset = FColor(180, 120, 110);
			static inline const FLinearColor NodeHeader = FColor(180, 120, 110);
			static inline const FLinearColor NodeBody = FColor(18, 12, 11, 127);
		};

		static void RegisterDataflowNodes()
		{
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("MeshResizing", FColorScheme::NodeHeader, FColorScheme::NodeBody);

			RegisterBaseBodyDataflowNodes();
			RegisterMeshConstraintDataflowNodes();
			RegisterMeshWrapNodes();
			RegisterUVUnwrapNodes();
			RegisterUVMeshTransformNodes();
			RegisterAlignUVMeshNodes();
			RegisterTextureNodes();

			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshWarpNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateRBFResizingWeightsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FApplyRBFResizingNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUVResizeControllerNode);
		}

	}  // End namespace Private

	class FMeshResizingDataflowNodesModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			Private::RegisterDataflowNodes();
		}

		virtual void ShutdownModule() override
		{
		}
	};
}  // End namespace UE::MeshResizing

IMPLEMENT_MODULE(UE::MeshResizing::FMeshResizingDataflowNodesModule, MeshResizingDataflowNodes)
