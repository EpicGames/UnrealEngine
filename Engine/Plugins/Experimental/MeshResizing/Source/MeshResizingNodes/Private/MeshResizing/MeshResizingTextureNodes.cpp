// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/MeshResizingTextureNodes.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshResizingTextureNodes)

namespace UE::MeshResizing
{
	void RegisterTextureNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshResizingGrowTileRegionNode);
	}

	namespace Private
	{
		// Function stolen from UVPacking.cpp
		// TODO: if and when this code is moved from experimental, eliminate this duplicate code
		template<int32 Dilate>
		void InternalRasterizeTriangle(TArray64<uint8>& RenderTarget, int32 RenderTargetWidth, int32 RenderTargetHeight, const FVector2f Points[3])
		{
			const FVector2f HalfPixel(0.5f, 0.5f);
			FVector2f p0 = Points[0] - HalfPixel;
			FVector2f p1 = Points[1] - HalfPixel;
			FVector2f p2 = Points[2] - HalfPixel;

			// Correct winding
			float Facing = (p0.X - p1.X) * (p2.Y - p0.Y) - (p0.Y - p1.Y) * (p2.X - p0.X);
			if (Facing < 0.0f)
			{
				Swap(p0, p2);
			}

			// 28.4 fixed point
			const int32 X0 = (int32)(16.0f * p0.X + 0.5f);
			const int32 X1 = (int32)(16.0f * p1.X + 0.5f);
			const int32 X2 = (int32)(16.0f * p2.X + 0.5f);

			const int32 Y0 = (int32)(16.0f * p0.Y + 0.5f);
			const int32 Y1 = (int32)(16.0f * p1.Y + 0.5f);
			const int32 Y2 = (int32)(16.0f * p2.Y + 0.5f);

			// Bounding rect
			int32 MinX = (FMath::Min3(X0, X1, X2) - Dilate + 15) / 16;
			int32 MaxX = (FMath::Max3(X0, X1, X2) + Dilate + 15) / 16;
			int32 MinY = (FMath::Min3(Y0, Y1, Y2) - Dilate + 15) / 16;
			int32 MaxY = (FMath::Max3(Y0, Y1, Y2) + Dilate + 15) / 16;

			// Clip to image
			MinX = FMath::Clamp(MinX, 0, RenderTargetWidth);
			MaxX = FMath::Clamp(MaxX, 0, RenderTargetWidth);
			MinY = FMath::Clamp(MinY, 0, RenderTargetHeight);
			MaxY = FMath::Clamp(MaxY, 0, RenderTargetHeight);

			// Deltas
			const int32 DX01 = X0 - X1;
			const int32 DX12 = X1 - X2;
			const int32 DX20 = X2 - X0;

			const int32 DY01 = Y0 - Y1;
			const int32 DY12 = Y1 - Y2;
			const int32 DY20 = Y2 - Y0;

			// Half-edge constants
			int32 C0 = DY01 * X0 - DX01 * Y0;
			int32 C1 = DY12 * X1 - DX12 * Y1;
			int32 C2 = DY20 * X2 - DX20 * Y2;

			// Correct for fill convention
			C0 += (DY01 < 0 || (DY01 == 0 && DX01 > 0)) ? 0 : -1;
			C1 += (DY12 < 0 || (DY12 == 0 && DX12 > 0)) ? 0 : -1;
			C2 += (DY20 < 0 || (DY20 == 0 && DX20 > 0)) ? 0 : -1;

			// Dilate edges
			C0 += (abs(DX01) + abs(DY01)) * Dilate;
			C1 += (abs(DX12) + abs(DY12)) * Dilate;
			C2 += (abs(DX20) + abs(DY20)) * Dilate;

			for (int32 y = MinY; y < MaxY; y++)
			{
				for (int32 x = MinX; x < MaxX; x++)
				{
					// same as Edge1>= 0 && Edge2>= 0 && Edge3>= 0
					int32 IsInside;
					IsInside = C0 + (DX01 * y - DY01 * x) * 16;
					IsInside |= C1 + (DX12 * y - DY12 * x) * 16;
					IsInside |= C2 + (DX20 * y - DY20 * x) * 16;

					if (IsInside >= 0)
					{
						RenderTarget[y + x * RenderTargetHeight] = 255;
					}
				}
			}
		}

		static void RasterizeUVMeshToMask(TArray64<uint8>& RenderTarget, int32 RenderTargetWidth, int32 RenderTargetHeight, const UE::Geometry::FDynamicMeshUVOverlay& UVOverlay)
		{
			RenderTarget.Init(0, RenderTargetWidth * RenderTargetHeight);

			for (int32 TID : UVOverlay.GetParentMesh()->TriangleIndicesItr())
			{
				FVector2f UVs[3];
				UVOverlay.GetTriElements(TID, UVs[0], UVs[1], UVs[2]);

				// Convert to image space
				for (int32 VID = 0; VID < 3; ++VID)
				{
					const FVector2f OldUVs = UVs[VID];
					UVs[VID][0] = OldUVs[1] * RenderTargetHeight;
					UVs[VID][1] = OldUVs[0] * RenderTargetWidth;
				}

				InternalRasterizeTriangle<0>(RenderTarget, RenderTargetWidth, RenderTargetHeight, UVs);
			}
		}

		static bool TileIsValid(const UE::Geometry::FIndex2i& TileStart, const UE::Geometry::FIndex2i& TileDim, const TArray64<uint8>& ValidMask, int32 ImageW, int32 ImageH)
		{
			if (ValidMask.Num() < (TileStart[0] + TileDim[0]) * (TileStart[1] + TileDim[1]))
			{
				return false;
			}

			for (int32 X = TileStart[0]; X < TileStart[0] + TileDim[0]; ++X)
			{
				for (int32 Y = TileStart[1]; Y < TileStart[1] + TileDim[1]; ++Y)
				{
					const bool bIsValid = (ValidMask[Y + ImageH * X] > 0);
					if (!bIsValid)
					{
						return false;
					}
				}
			}

			return true;
		}

		static void GetTileData(TArray<FLinearColor>& OutTile, const UE::Geometry::FIndex2i& TileStart, const UE::Geometry::FIndex2i& TileDim, const FImage& FullImage)
		{
			OutTile.Reserve(TileDim[0] * TileDim[1]);

			TArrayView64<const FLinearColor> ImageData = FullImage.AsRGBA32F();

			const int32 MaxX = FMath::Min(TileStart[0] + TileDim[0], FullImage.GetWidth());
			const int32 MaxY = FMath::Min(TileStart[1] + TileDim[1], FullImage.GetHeight());

			for (int32 X = TileStart[0]; X < MaxX; ++X)
			{
				for (int32 Y = TileStart[1]; Y < MaxY; ++Y)
				{
					OutTile.Add(ImageData[Y + X * FullImage.GetHeight()]);
				}
			}
		}

		static void StampTileData(TArray<FLinearColor>& ImageBuffer, int32 ImageW, int32 ImageH, const TArray<FLinearColor>& Tile, const UE::Geometry::FIndex2i& TileStart, const UE::Geometry::FIndex2i& TileDim)
		{
			const int32 MaxX = FMath::Min(TileDim[0], ImageW - TileStart[0]);
			const int32 MaxY = FMath::Min(TileDim[1], ImageH - TileStart[1]);

			for (int32 X = 0; X < MaxX; ++X)
			{
				for (int32 Y = 0; Y < MaxY; ++Y)
				{
					ImageBuffer[(TileStart[1] + Y) + (TileStart[0] + X) * ImageH] = Tile[Y + X * TileDim[1]];
				}
			}
		}
	}

}	// namespace UE::MeshResizing

FMeshResizingGrowTileRegionNode::FMeshResizingGrowTileRegionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Image);
	RegisterInputConnection(&ValidRegionMesh);
	RegisterOutputConnection(&Image, &Image);
	RegisterOutputConnection(&MeshMask);
}

void FMeshResizingGrowTileRegionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Image) || Out->IsA(&MeshMask))
	{
		const FDataflowImage InImage = GetValue(Context, &Image);
		const int32 ImageW = InImage.GetWidth();
		const int32 ImageH = InImage.GetHeight();
		const TObjectPtr<const UDataflowMesh> Mesh = GetValue(Context, &ValidRegionMesh);

		if (Mesh)
		{
			if (const UE::Geometry::FDynamicMesh3* const DynamicMesh = Mesh->GetDynamicMesh())
			{
				if (DynamicMesh->HasAttributes() && DynamicMesh->Attributes()->NumUVLayers() > MeshUVLayer && MeshUVLayer >= 0)
				{
					if (ImageW > 0 && ImageH > 0)
					{
						TArray64<uint8> ValidMask;
						UE::MeshResizing::Private::RasterizeUVMeshToMask(ValidMask, ImageW, ImageH, *DynamicMesh->Attributes()->GetUVLayer(MeshUVLayer));

						if (Out->IsA(&MeshMask))
						{
							// Write out just the image mask
							FDataflowImage OutMaskImage;
							OutMaskImage.CreateRGBA32F(ImageW, ImageH);
							const FImageView SrcImage(ValidMask.GetData(), ImageW, ImageH, /*NumSlices*/1, ERawImageFormat::Type::G8, EGammaSpace::Linear);
							FImageCore::CopyImage(SrcImage, OutMaskImage.GetImage());
							SetValue(Context, OutMaskImage, &MeshMask);
							return;
						}

						if (TileWidth > 0)
						{
							const UE::Geometry::FIndex2i TileSize(TileWidth, TileWidth);

							const int32 NumTilesX = ImageW / TileWidth;
							const int32 NumTilesY = ImageH / TileWidth;

							UE::Geometry::FIndex2i CurrentTileStart(0, 0);

							bool bFoundValidTile = false;
							for (int32 TileX = 0; TileX < NumTilesX; ++TileX)
							{
								for (int32 TileY = 0; TileY < NumTilesY; ++TileY)
								{
									CurrentTileStart = UE::Geometry::FIndex2i(TileX * TileWidth, TileY * TileWidth);
									bFoundValidTile = UE::MeshResizing::Private::TileIsValid(CurrentTileStart, TileSize, ValidMask, ImageW, ImageH);

									if (bFoundValidTile)
									{
										break;
									}
								}
							}

							if (bFoundValidTile)
							{
								TArray<FLinearColor> TileBuffer;
								UE::MeshResizing::Private::GetTileData(TileBuffer, CurrentTileStart, TileSize, InImage.GetImage());

								TArray<FLinearColor> OutImageBuffer;
								OutImageBuffer.SetNumUninitialized(ImageW * ImageH);

								for (int32 TileX = 0; TileX < NumTilesX + 1; ++TileX)
								{
									for (int32 TileY = 0; TileY < NumTilesY + 1; ++TileY)
									{
										UE::MeshResizing::Private::StampTileData(OutImageBuffer, ImageW, ImageH, TileBuffer, UE::Geometry::FIndex2i(TileX * TileWidth, TileY * TileWidth), TileSize);
									}
								}

								const FImageView OutImageView(OutImageBuffer.GetData(), ImageW, ImageH, /*NumSlices*/1, ERawImageFormat::Type::RGBA32F, EGammaSpace::Linear);

								FDataflowImage OutImage;
								OutImage.CreateRGBA32F(ImageW, ImageH);
								FImageCore::CopyImage(OutImageView, OutImage.GetImage());

								SetValue(Context, OutImage, &Image);
								return;
							}
							else
							{
								Context.Warning(TEXT("Did not find valid Tile of the specified size in the UV region specified by ValidRegionMesh"), this, Out);
							}
						}
						else
						{
							Context.Warning(TEXT("TileWidth is zero"), this, Out);
						}
					}
					else
					{
						Context.Warning(TEXT("Image has zero dimension"), this, Out);
					}
				}
				else
				{
					Context.Warning(TEXT("ValidRegionMesh has no UV Layer corresponding to index MeshUVLayer"), this, Out);
				}
			}
			else
			{
				Context.Warning(TEXT("ValidRegionMesh has no DynamicMesh object"), this, Out);
			}
		}
	}

	const FDataflowImage DefaultMaskImage;
	SetValue(Context, DefaultMaskImage, &MeshMask);

	SafeForwardInput(Context, &Image, &Image);
}

