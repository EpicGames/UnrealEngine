// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowImage.h"
#include "MeshResizingTextureNodes.generated.h"

class UDataflowMesh;

/**
 * Finds a square tile within a specified image region and duplicates it over the whole image.
 * The image region to search is determined by the UV coordinates in ValidRegionMesh -- only texels inside a 2D UV mesh triangle are considered when searching for a tile.
 * Note this node does not try to detect any repeating patterns, it just grabs the first square tile of the specified size that is entirely inside the UV mesh.
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FMeshResizingGrowTileRegionNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshResizingGrowTileRegionNode, "GrowTileRegion", "MeshResizing", "Grow Tile")

public:

	FMeshResizingGrowTileRegionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowIntrinsic, DataflowInput, DataflowOutput, DataflowPassthrough = "Image"))
	FDataflowImage Image;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> ValidRegionMesh;

	UPROPERTY(EditAnywhere, Category = "Valid Region", meta = (UIMin = 0, ClampMin = 0))
	int MeshUVLayer = 0;

	UPROPERTY(EditAnywhere, Category = "Tile", meta = (UIMin = 1, ClampMin = 1))
	int TileWidth = 10;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowImage MeshMask;


	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

namespace UE::MeshResizing
{
	void RegisterTextureNodes();
}
