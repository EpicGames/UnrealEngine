// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshResizing/MeshWarpNode.h"
#include "MeshResizing/RBFInterpolation.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMeshEditor.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowMesh.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshWarpNode)

using UE::Geometry::FDynamicMesh3;

namespace  UE::MeshResizing::Private
{
	void BlendMesh(
		FDynamicMesh3& OutMesh,
		const FDynamicMesh3& SourceMesh,
		const FDynamicMesh3& TargetMesh,
		float Alpha)
	{
		OutMesh.Copy(SourceMesh);

		ensure(SourceMesh.VertexCount() == TargetMesh.VertexCount());
		ensure(SourceMesh.VertexCount() == OutMesh.VertexCount());

		for (const int32 VertexIndex : TargetMesh.VertexIndicesItr())
		{
			const FVector3d SourceVert = SourceMesh.GetVertex(VertexIndex);
			const FVector3d TargetVert = TargetMesh.GetVertex(VertexIndex);
			const FVector3d FinalPoint = (1.0 - Alpha) * SourceVert + Alpha * TargetVert;
			OutMesh.SetVertex(VertexIndex, FinalPoint);
		}
	}


	void ApplyRefitToMesh_WrapDeform(
		FDynamicMesh3& ResizedMesh,
		const FDynamicMesh3& SourceMesh,
		const FDynamicMesh3& TargetMesh,
		float Alpha)
	{
		using namespace UE::Geometry;

		FDynamicMeshAABBTree3 SourceTree(&SourceMesh);

		struct FEmbeddingInfo
		{
			int ClosestTriangle;
			FVector4d Weights;
		};

		TArray<FEmbeddingInfo> EmbeddingInfo;
		EmbeddingInfo.SetNum(ResizedMesh.VertexCount());

		for (const int32 VertexIndex : ResizedMesh.VertexIndicesItr())
		{
			EmbeddingInfo[VertexIndex].ClosestTriangle = INDEX_NONE;

			const FVector3d VertexLocation = ResizedMesh.GetVertex(VertexIndex);

			double NearestDistSqr;
			const int32 NearTriID = SourceTree.FindNearestTriangle(VertexLocation, NearestDistSqr);
			if (NearTriID >= 0)
			{
				const FVector3d Normal = SourceMesh.GetTriNormal(NearTriID);
				FVector3d TriPointA, TriPointB, TriPointC;
				SourceMesh.GetTriVertices(NearTriID, TriPointA, TriPointB, TriPointC);
				const FPlane3d TriPlane(Normal, TriPointA);
				const double Dist = TriPlane.DistanceTo(VertexLocation);
				const FVector3d PointOnPlane = VertexLocation - Dist * Normal;
				const FVector Bary = VectorUtil::BarycentricCoords(PointOnPlane, TriPointA, TriPointB, TriPointC);

				EmbeddingInfo[VertexIndex].ClosestTriangle = NearTriID;
				EmbeddingInfo[VertexIndex].Weights = { Bary, Dist };
			}
		}

		for (const int32 VertexIndex : ResizedMesh.VertexIndicesItr())
		{
			if (EmbeddingInfo[VertexIndex].ClosestTriangle != INDEX_NONE)
			{
				const FVector3d Normal = TargetMesh.GetTriNormal(EmbeddingInfo[VertexIndex].ClosestTriangle);
				FVector3d TriPointA, TriPointB, TriPointC;
				TargetMesh.GetTriVertices(EmbeddingInfo[VertexIndex].ClosestTriangle, TriPointA, TriPointB, TriPointC);

				const FVector4d& Weights = EmbeddingInfo[VertexIndex].Weights;
				const FVector PointOnPlane = Weights[0] * TriPointA + Weights[1] * TriPointB + Weights[2] * TriPointC;
				const FVector FinalPoint = PointOnPlane + Weights[3] * Normal;

				ResizedMesh.SetVertex(VertexIndex, FinalPoint);
			}
		}
	}

	void ApplyRefitToMesh_RBFInterpolate(
		FDynamicMesh3& ResizedMesh,
		const FDynamicMesh3& SourceMesh,
		const FDynamicMesh3& TargetMesh,
		int32 NumInterpolationPoints,
		bool bInterpolateNormals)
	{
		if (ResizedMesh.VertexCount() == 0)
		{
			return;
		}

		FMeshResizingRBFInterpolationData InterpolationData;
		FRBFInterpolation::GenerateWeights(SourceMesh, NumInterpolationPoints, InterpolationData);

		// Get target mesh samples
		TArray<FVector3f> TargetSamplePoints;
		TargetSamplePoints.Reserve(InterpolationData.SampleIndices.Num());
		for (const uint32 SampleIndex : InterpolationData.SampleIndices)
		{
			TargetSamplePoints.Emplace(FVector3f(TargetMesh.GetVertex(SampleIndex)));
		}
		// Do the interpolation
		FRBFInterpolation::DeformPoints(TargetSamplePoints, InterpolationData, bInterpolateNormals, ResizedMesh);
	}
}


FMeshWarpNode::FMeshWarpNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid) :
	FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MeshToResize);
	RegisterInputConnection(&SourceMesh);
	RegisterInputConnection(&TargetMesh);

	RegisterOutputConnection(&BlendedTargetMesh, &SourceMesh);
	RegisterOutputConnection(&ResizedMesh, &MeshToResize);
}

void FMeshWarpNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;


	if (TObjectPtr<UDataflowMesh> InMeshToResize = GetValue(Context, &MeshToResize))
	{
		if (TObjectPtr<UDataflowMesh> InSourceMesh = GetValue(Context, &SourceMesh))
		{
			if (TObjectPtr<UDataflowMesh> InTargetMesh = GetValue(Context, &TargetMesh))
			{
				if (InMeshToResize->GetDynamicMesh() && InSourceMesh->GetDynamicMesh() && InTargetMesh->GetDynamicMesh())
				{
					TObjectPtr<UDataflowMesh> OutResizedMesh = NewObject<UDataflowMesh>();
					TObjectPtr<UDataflowMesh> OutBlendedTargetMesh = NewObject<UDataflowMesh>();
					FDynamicMesh3 BlendedTargetFMesh;
					UE::MeshResizing::Private::BlendMesh(BlendedTargetFMesh, InSourceMesh->GetDynamicMeshRef(), InTargetMesh->GetDynamicMeshRef(), Alpha);

					FDynamicMesh3 ResizeMesh;
					ResizeMesh.Copy(InMeshToResize->GetDynamicMeshRef());
					switch (WarpMethod)
					{
					case EMeshResizingWarpMethod::RBFInterpolate:
					default:
						UE::MeshResizing::Private::ApplyRefitToMesh_RBFInterpolate(ResizeMesh, InSourceMesh->GetDynamicMeshRef(), BlendedTargetFMesh, NumInterpolationPoints, bInterpolateNormals);
						break;
					case EMeshResizingWarpMethod::WrapDeform:
						UE::MeshResizing::Private::ApplyRefitToMesh_WrapDeform(ResizeMesh, InSourceMesh->GetDynamicMeshRef(), BlendedTargetFMesh, 1.0f);
						break;
					}

					OutBlendedTargetMesh->SetDynamicMesh(MoveTemp(BlendedTargetFMesh));
					OutResizedMesh->SetDynamicMesh(MoveTemp(ResizeMesh));
					OutBlendedTargetMesh->SetMaterials(InTargetMesh->GetMaterials());
					OutResizedMesh->SetMaterials(InMeshToResize->GetMaterials());
					SetValue(Context, OutBlendedTargetMesh, &BlendedTargetMesh);
					SetValue(Context, OutResizedMesh, &ResizedMesh);
					return;
				}
			}
		}
	}

	SafeForwardInput(Context, &MeshToResize, &ResizedMesh);
	SafeForwardInput(Context, &SourceMesh, &BlendedTargetMesh);
}
