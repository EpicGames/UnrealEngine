// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "MeshWarpNode.generated.h"

class UDataflowMesh;
class UMaterialInterface;

UENUM()
enum struct EMeshResizingWarpMethod : uint8
{
	WrapDeform,
	RBFInterpolate
};

USTRUCT(Meta = (MeshResizing, Experimental))
struct FMeshWarpNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshWarpNode, "MeshWarp", "MeshResizing", "Mesh Warp")
	DATAFLOW_NODE_RENDER_TYPE_START()
	DATAFLOW_NODE_RENDER_TYPE_ADD("SurfaceRender", FName("UDataflowMesh"), "BlendedTargetMesh")
	DATAFLOW_NODE_RENDER_TYPE_ADD("SurfaceRender", FName("UDataflowMesh"), "ResizedMesh")
	DATAFLOW_NODE_RENDER_TYPE_END()

public:

	FMeshWarpNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> MeshToResize;

	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> SourceMesh;

	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> TargetMesh;


	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "SourceMesh"))
	TObjectPtr<UDataflowMesh> BlendedTargetMesh;

	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "MeshToResize"))
	TObjectPtr<UDataflowMesh> ResizedMesh;

	UPROPERTY(EditAnywhere, Category = "Warp", meta = (Min = 0.0f, UIMin = 0.0f, Max = 1.0f, UIMax = 1.0f))
	float Alpha = 1.f;

	UPROPERTY(EditAnywhere, Category = "Warp")
	EMeshResizingWarpMethod WarpMethod = EMeshResizingWarpMethod::RBFInterpolate;

	UPROPERTY(EditAnywhere, Category = "Warp", meta =  (Min = 1, EditCondition = "WarpMethod == EMeshResizingWarpMethod::RBFInterpolate", EditConditionHides))
	int32 NumInterpolationPoints = 100;

	UPROPERTY(EditAnywhere, Category = "Warp", meta = (EditCondition = "WarpMethod == EMeshResizingWarpMethod::RBFInterpolate", EditConditionHides))
	bool bInterpolateNormals = true;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};
