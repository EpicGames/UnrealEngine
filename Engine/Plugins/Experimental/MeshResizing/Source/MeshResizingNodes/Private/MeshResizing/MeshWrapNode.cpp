// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshResizing/MeshWrapNode.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowMesh.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowObject.h"
#if WITH_EDITOR
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#endif
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshWrapNode)

using UE::Geometry::FDynamicMesh3;

namespace UE::MeshResizing
{
	void RegisterMeshWrapNodes()
	{
		static const FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshWrapNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshWrapLandmarksNode);
	}
}

namespace UE::MeshResizing::Private
{
#if WITH_EDITOR
	static FLinearColor PseudoRandomColor(int32 NumColorRotations)
	{
		constexpr uint8 Spread = 157;  // Prime number that gives a good spread of colors without getting too similar as a rand might do.
		uint8 Seed = Spread;
		NumColorRotations = FMath::Abs(NumColorRotations);
		for (int32 Rotation = 0; Rotation < NumColorRotations; ++Rotation)
		{
			Seed += Spread;
		}
		return FLinearColor::MakeFromHSV8(Seed, 180, 140);
	}
#endif

	static TObjectPtr<UDynamicMeshComponent> MakeDynamicMeshComponent(const TObjectPtr<UDataflowMesh> Mesh, const FName& MeshName, const TObjectPtr<AActor> RootActor)
	{
		static UMaterial* DefaultMaterial = nullptr;
		if (!DefaultMaterial)
		{
			// Use dataflow's default material
			DefaultMaterial = Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Dataflow/DataflowTwoSidedVertexMaterial")));
		}

		if (Mesh)
		{
			if (Mesh->GetDynamicMesh())
			{
				FDynamicMesh3 WrappedMeshCopy;
				WrappedMeshCopy.Copy(Mesh->GetDynamicMeshRef());
				TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = NewObject<UDynamicMeshComponent>(RootActor);
				DynamicMeshComponent->SetMesh(MoveTemp(WrappedMeshCopy));
				if (Mesh->GetMaterials().IsEmpty())
				{
					DynamicMeshComponent->SetOverrideRenderMaterial(DefaultMaterial);
				}
				else
				{
					DynamicMeshComponent->ConfigureMaterialSet(Mesh->GetMaterials());
				}
				return DynamicMeshComponent;
			}
		}
		return nullptr;
	}
}

FMeshWrapLandmarksNode::FMeshWrapLandmarksNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid) :
	FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterOutputConnection(&Mesh, &Mesh);
	RegisterOutputConnection(&Landmarks);
}

void FMeshWrapLandmarksNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Landmarks))
	{
		SetValue(Context, Landmarks, &Landmarks);
	}
	if (Out->IsA(&Mesh))
	{
		SafeForwardInput(Context, &Mesh, &Mesh);
	}
}

#if WITH_EDITOR
bool FMeshWrapLandmarksNode::CanDebugDrawViewMode(const FName& ViewMode) const
{
	return ViewMode == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FMeshWrapLandmarksNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	DataflowRenderingInterface.SetPointSize(PointSize);
	if (bCanDebugDraw && (DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		if (TObjectPtr<UDataflowMesh> InMesh = GetValue(Context, &Mesh))
		{
			if (InMesh->GetDynamicMesh())
			{
				const TArray<FMeshWrapLandmark>& OutLandmarks = GetOutputValue(Context, &Landmarks, Landmarks);
				DataflowRenderingInterface.ReservePoints(OutLandmarks.Num());

				for (int32 Index = 0; Index < OutLandmarks.Num(); ++Index)
				{
					if (InMesh->GetDynamicMeshRef().IsVertex(OutLandmarks[Index].VertexIndex))
					{
						DataflowRenderingInterface.SetColor(UE::MeshResizing::Private::PseudoRandomColor(Index));
						const FVector3d& Point = InMesh->GetDynamicMeshRef().GetVertexRef(OutLandmarks[Index].VertexIndex);
						DataflowRenderingInterface.DrawPoint(Point);
						if (bShowIndex || bShowIdentifier)
						{
							const FString IndexString = bShowIndex ? FString::FromInt(Index) : FString();

							DataflowRenderingInterface.DrawText3d(IndexString + FString(" ") + OutLandmarks[Index].Identifier, Point);
						}
					}
				}
			}
		}
	}
}
#endif

// Object encapsulating a change to the Selection Node's values. Used for Undo/Redo.
class FMeshWrapLandmarksNode::FLandmarksNodeChange final : public FToolCommandChange
{
public:
	FLandmarksNodeChange(const FMeshWrapLandmarksNode& Node)
		: NodeGuid(Node.GetGuid())
		, SavedLandmarks(Node.Landmarks)
	{}

private:
	FGuid NodeGuid;
	TArray<FMeshWrapLandmark> SavedLandmarks;

	virtual FString ToString() const final { return TEXT("MeshWrapLandmarksNodeChange"); }

	virtual void Apply(UObject* Object) final { SwapApplyRevert(Object); }
	virtual void Revert(UObject* Object) final { SwapApplyRevert(Object); }

	void SwapApplyRevert(UObject* Object)
	{
		if (UDataflow* const Dataflow = Cast<UDataflow>(Object))
		{
			if (const TSharedPtr<FDataflowNode> BaseNode = Dataflow->GetDataflow()->FindBaseNode(NodeGuid))
			{
				if (FMeshWrapLandmarksNode* const Node = BaseNode->AsType<FMeshWrapLandmarksNode>())
				{
					Swap(Node->Landmarks, SavedLandmarks);
					Node->Invalidate();
				}
			}
		}
	}
};

TUniquePtr<class FToolCommandChange> FMeshWrapLandmarksNode::MakeSelectedNodeChange(const FMeshWrapLandmarksNode& Node)
{
	return MakeUnique<FMeshWrapLandmarksNode::FLandmarksNodeChange>(Node);
}

FMeshWrapNode::FMeshWrapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid) :
	FDataflowPrimitiveNode(InParam, InGuid)
{
	RegisterInputConnection(&SourceTopologyMesh);
	RegisterInputConnection(&TargetShapeMesh);
	RegisterInputConnection(&SourceTopologyLandmarks);
	RegisterInputConnection(&TargetShapeLandmarks);

	RegisterOutputConnection(&WrappedMesh, &SourceTopologyMesh);
	RegisterOutputConnection(&MatchedLandmarks);
}

void FMeshWrapNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA<TArray<FMeshWrapCorrespondence>>(&MatchedLandmarks))
	{
		SetValue(Context, CalculateMatchedLandmarks(Context), &MatchedLandmarks);
		return;
	}

	if (Out->IsA(&WrappedMesh))
	{
		if (TObjectPtr<UDataflowMesh> InSourceTopologyMesh = GetValue(Context, &SourceTopologyMesh))
		{
			if (TObjectPtr<UDataflowMesh> InTargetShapeMesh = GetValue(Context, &TargetShapeMesh))
			{
				if (InSourceTopologyMesh->GetDynamicMesh() && InTargetShapeMesh->GetDynamicMesh())
				{
					const TArray<FMeshWrapCorrespondence>& InMatchedLandmarks = GetOutputValue(Context, &MatchedLandmarks, TArray<FMeshWrapCorrespondence>());
					TArray<FWrapMeshCorrespondence> Correspondences;
					Correspondences.Reserve(InMatchedLandmarks.Num());
					for (const FMeshWrapCorrespondence& MatchedLandmark : InMatchedLandmarks)
					{
						Correspondences.Emplace(MatchedLandmark.SourceVertexIndex, MatchedLandmark.TargetVertexIndex);
					}
					TObjectPtr<UDataflowMesh> OutWrappedMesh = NewObject<UDataflowMesh>();
					FDynamicMesh3 WrappedFMesh;
					FWrapMesh MeshWrapper(nullptr);
					MeshWrapper.MaxNumOuterIterations = MaxNumOuterIterations;
					MeshWrapper.NumInnerIterations = NumInnerIterations;
					MeshWrapper.ProjectionTolerance = ProjectionTolerance;
					MeshWrapper.LaplacianStiffness = LaplacianStiffness;
					MeshWrapper.InitialProjectionStiffness = InitialProjectionStiffness;
					MeshWrapper.ProjectionStiffnessMuliplier = ProjectionStiffnessMuliplier;
					MeshWrapper.CorrespondenceStiffness = CorrespondenceStiffness;
					MeshWrapper.LaplacianType = FWrapMesh::ELaplacianType::AffineInvariant;
					MeshWrapper.SetMesh(InSourceTopologyMesh->GetDynamicMesh());

					MeshWrapper.WrapToTargetShape(InTargetShapeMesh->GetDynamicMeshRef(), Correspondences, WrappedFMesh);
					OutWrappedMesh->SetDynamicMesh(MoveTemp(WrappedFMesh));
					OutWrappedMesh->SetMaterials(InSourceTopologyMesh->GetMaterials());
					SetValue(Context, OutWrappedMesh, &WrappedMesh);

					return;
				}
			}

		}

		SafeForwardInput(Context, &SourceTopologyMesh, &WrappedMesh);
	}
}

#if WITH_EDITOR
bool FMeshWrapNode::CanDebugDrawViewMode(const FName& ViewMode) const
{
	return ViewMode == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FMeshWrapNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	constexpr float PointSize = 5.f;
	DataflowRenderingInterface.SetPointSize(PointSize);
	if (DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned)
	{
		if (TObjectPtr<UDataflowMesh> OutWrappedMesh = GetOutputValue(Context, &WrappedMesh, WrappedMesh))
		{
			if (TObjectPtr<UDataflowMesh> InTargetShapeMesh = GetValue(Context, &TargetShapeMesh))
			{
				if (TObjectPtr<UDataflowMesh> InSourceTopologyMesh = GetValue(Context, &SourceTopologyMesh))
				{
					if (OutWrappedMesh->GetDynamicMesh() && InTargetShapeMesh->GetDynamicMesh() && InSourceTopologyMesh->GetDynamicMesh())
					{
						const TArray<FMeshWrapCorrespondence>& InSourceTargetCorrespondences = GetOutputValue(Context, &MatchedLandmarks, TArray<FMeshWrapCorrespondence>());
						DataflowRenderingInterface.ReservePoints((2 + (bDisplaySource ? 1 : 0) + (bDisplayTarget ? 1 : 0)) * InSourceTargetCorrespondences.Num());

						for (int32 Index = 0; Index < InSourceTargetCorrespondences.Num(); ++Index)
						{
							const FMeshWrapCorrespondence& Correspondence = InSourceTargetCorrespondences[Index];
							if (OutWrappedMesh->GetDynamicMeshRef().IsVertex(Correspondence.SourceVertexIndex) && InTargetShapeMesh->GetDynamicMeshRef().IsVertex(Correspondence.TargetVertexIndex) && InSourceTopologyMesh->GetDynamicMeshRef().IsVertex(Correspondence.SourceVertexIndex))
							{
								DataflowRenderingInterface.SetColor(UE::MeshResizing::Private::PseudoRandomColor(Index));
								const FVector3d& WrappedPoint = OutWrappedMesh->GetDynamicMeshRef().GetVertexRef(Correspondence.SourceVertexIndex);
								const FVector3d& TargetPoint = InTargetShapeMesh->GetDynamicMeshRef().GetVertexRef(Correspondence.TargetVertexIndex);
								DataflowRenderingInterface.DrawPoint(WrappedPoint);
								DataflowRenderingInterface.DrawPoint(TargetPoint);
								DataflowRenderingInterface.DrawLine(WrappedPoint, TargetPoint);
								const FString IndexString = FString::FromInt(Index) + FString(" ") + Correspondence.Identifier;
								DataflowRenderingInterface.DrawText3d(IndexString, WrappedPoint);

								if (bDisplaySource)
								{
									const FVector3d OffsetSourcePoint = InSourceTopologyMesh->GetDynamicMeshRef().GetVertexRef(Correspondence.SourceVertexIndex) + FVector3d(SourceDisplayOffset, 0., 0.);
									DataflowRenderingInterface.DrawPoint(OffsetSourcePoint);
									DataflowRenderingInterface.DrawText3d(IndexString, OffsetSourcePoint);
								}

								if(bDisplayTarget)
								{
									const FVector3d OffsetTargetPoint = TargetPoint + FVector3d(TargetDisplayOffset, 0., 0.);
									DataflowRenderingInterface.DrawPoint(OffsetTargetPoint);
									DataflowRenderingInterface.DrawText3d(IndexString, OffsetTargetPoint);
								}
							}
						}
					}
				}
			}
		}
	}
}
#endif

void FMeshWrapNode::AddPrimitiveComponents(UE::Dataflow::FContext& Context, const TSharedPtr<const FManagedArrayCollection> RenderCollection, TObjectPtr<UObject> NodeOwner,
	TObjectPtr<AActor> RootActor, TArray<TObjectPtr<UPrimitiveComponent>>& PrimitiveComponents)
{
	if (bDisplaySource)
	{
		static const FName SourceMeshName(TEXT("SourceTopologyMesh"));
		if (TObjectPtr<UDynamicMeshComponent> SourceComp = UE::MeshResizing::Private::MakeDynamicMeshComponent(GetValue(Context, &SourceTopologyMesh), SourceMeshName, RootActor))
		{
			SourceComp->SetWorldTransform(FTransform(FVector3d(SourceDisplayOffset, 0., 0.)));
			PrimitiveComponents.Add(SourceComp);
		}
	}

	if (bDisplayTarget)
	{
		static const FName TargetMeshName(TEXT("TargetShapeMesh"));
		if (TObjectPtr<UDynamicMeshComponent> TargetComp = UE::MeshResizing::Private::MakeDynamicMeshComponent(GetValue(Context, &TargetShapeMesh), TargetMeshName, RootActor))
		{
			TargetComp->SetWorldTransform(FTransform(FVector3d(TargetDisplayOffset, 0., 0.)));
			PrimitiveComponents.Add(TargetComp);
		}
	}
}

TArray<FMeshWrapCorrespondence> FMeshWrapNode::CalculateMatchedLandmarks(UE::Dataflow::FContext& Context) const
{
	const TArray<FMeshWrapLandmark>& InSourceLandmarks = GetValue(Context, &SourceTopologyLandmarks);
	const TArray<FMeshWrapLandmark>& InTargetLandmarks = GetValue(Context, &TargetShapeLandmarks);

	// Convert Source to a Map for faster lookup
	TMap<FString, int32> SourceLandmarkMap;
	SourceLandmarkMap.Reserve(InSourceLandmarks.Num());
	for (const FMeshWrapLandmark& SourceLandmark : InSourceLandmarks)
	{
		SourceLandmarkMap.Add(SourceLandmark.Identifier, SourceLandmark.VertexIndex);
	}

	TArray<FMeshWrapCorrespondence> Result;
	Result.Reserve(InSourceLandmarks.Num());

	for (const FMeshWrapLandmark& TargetLandmark : InTargetLandmarks)
	{
		if (const int32* SourceVertex = SourceLandmarkMap.Find(TargetLandmark.Identifier))
		{
			FMeshWrapCorrespondence& Correspondence = Result.AddDefaulted_GetRef();
			Correspondence.Identifier = TargetLandmark.Identifier;
			Correspondence.SourceVertexIndex = *SourceVertex;
			Correspondence.TargetVertexIndex = TargetLandmark.VertexIndex;
		}
	}

	return Result;
}