// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshResizing/RBFInterpolationNodes.h"
#include "MeshResizing/RBFInterpolation.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMeshEditor.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowMesh.h"
#include "Materials/MaterialInterface.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif
#if WITH_EDITORONLY_DATA
#include "Engine/SkeletalMesh.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RBFInterpolationNodes)

using UE::Geometry::FDynamicMesh3;

FGenerateRBFResizingWeightsNode::FGenerateRBFResizingWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid) :
	FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&SourceMesh);
	RegisterInputConnection(&NumInterpolationPoints);
	RegisterInputConnection(&MeshToResize);

	RegisterOutputConnection(&InterpolationData);
}

void FGenerateRBFResizingWeightsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	if (Out->IsA<FMeshResizingRBFInterpolationData>(&InterpolationData))
	{
		FMeshResizingRBFInterpolationData Result;
		if (TObjectPtr<UDataflowMesh> InSourceMesh = GetValue(Context, &SourceMesh))
		{
			if (InSourceMesh->GetDynamicMesh())
			{
				UE::MeshResizing::FRBFInterpolation::GenerateWeights(InSourceMesh->GetDynamicMeshRef(), GetValue(Context, &NumInterpolationPoints), Result);
			}
		}

		SetValue(Context, MoveTemp(Result), &InterpolationData);
	}
}

#if WITH_EDITOR
bool FGenerateRBFResizingWeightsNode::CanDebugDrawViewMode(const FName& ViewMode) const
{
	return ViewMode == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FGenerateRBFResizingWeightsNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if (DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned)
	{
		const FMeshResizingRBFInterpolationData& OutInterpolationData = GetOutputValue(Context, &InterpolationData, InterpolationData);
		if (OutInterpolationData.SampleIndices.Num())
		{
			DataflowRenderingInterface.SetColor(FLinearColor::Yellow);
			DataflowRenderingInterface.SetPointSize(2.f);

			DataflowRenderingInterface.ReservePoints(OutInterpolationData.SampleRestPositions.Num());
			for (const FVector3f& Pos : OutInterpolationData.SampleRestPositions)
			{
				DataflowRenderingInterface.DrawPoint(FVector(Pos));
			}
		}
	}
}
#endif

FApplyRBFResizingNode::FApplyRBFResizingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid) :
	FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MeshToResize);
	RegisterInputConnection(&TargetSkeletalMesh);
	RegisterInputConnection(&TargetSkeletalMeshLODIndex);
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&InterpolationData);

	RegisterOutputConnection(&ResizedMesh, &MeshToResize);
}


void FApplyRBFResizingNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<const UDataflowMesh>>(&ResizedMesh))
	{
		if (TObjectPtr<const UDataflowMesh> InMeshToResize = GetValue(Context, &MeshToResize))
		{
#if WITH_EDITORONLY_DATA
			if (bUseSkeletalMeshTarget)
			{
				if (TObjectPtr<const USkeletalMesh> InTargetSKM = GetValue(Context, &TargetSkeletalMesh))
				{
					const int32 InLODIndex = GetValue(Context, &TargetSkeletalMeshLODIndex);
					if (const FMeshDescription* const MeshDescription = InTargetSKM->GetMeshDescription(InLODIndex))
					{
						const FMeshResizingRBFInterpolationData& InInterpolationData = GetValue(Context, &InterpolationData);
						if (InMeshToResize->GetDynamicMesh() && InInterpolationData.SampleIndices.Num())
						{
							TObjectPtr<UDataflowMesh> OutResizedMesh = NewObject<UDataflowMesh>();
							FDynamicMesh3 ResizeMesh;
							ResizeMesh.Copy(InMeshToResize->GetDynamicMeshRef());

							// Do the interpolation 
							UE::MeshResizing::FRBFInterpolation::DeformPoints(*MeshDescription, InInterpolationData, bInterpolateNormals, ResizeMesh);
							OutResizedMesh->SetDynamicMesh(MoveTemp(ResizeMesh));
							OutResizedMesh->SetMaterials(InMeshToResize->GetMaterials());
							SetValue<TObjectPtr<const UDataflowMesh>>(Context, OutResizedMesh, &ResizedMesh);
							return;
						}
					}
				}
			}
			else
#endif
			{
				if (TObjectPtr<const UDataflowMesh> InTargetMesh = GetValue(Context, &TargetMesh))
				{
					const FMeshResizingRBFInterpolationData& InInterpolationData = GetValue(Context, &InterpolationData);

					if (InMeshToResize->GetDynamicMesh() && InTargetMesh->GetDynamicMesh() && InInterpolationData.SampleIndices.Num())
					{
						TObjectPtr<UDataflowMesh> OutResizedMesh = NewObject<UDataflowMesh>();
						FDynamicMesh3 ResizeMesh;
						ResizeMesh.Copy(InMeshToResize->GetDynamicMeshRef());

						const FDynamicMesh3& TargetMeshRef = InTargetMesh->GetDynamicMeshRef();

						// Get target mesh samples
						TArray<FVector3f> TargetSamplePoints;
						TargetSamplePoints.Reserve(InInterpolationData.SampleIndices.Num());
						for (const uint32 SampleIndex : InInterpolationData.SampleIndices)
						{
							TargetSamplePoints.Emplace(FVector3f(TargetMeshRef.GetVertex(SampleIndex)));
						}
						// Do the interpolation
						UE::MeshResizing::FRBFInterpolation::DeformPoints(TargetSamplePoints, InInterpolationData, bInterpolateNormals, ResizeMesh);
						OutResizedMesh->SetDynamicMesh(MoveTemp(ResizeMesh));
						OutResizedMesh->SetMaterials(InMeshToResize->GetMaterials());
						SetValue<TObjectPtr<const UDataflowMesh>>(Context, OutResizedMesh, &ResizedMesh);
						return;
					}
				}
			}
		}

		SafeForwardInput(Context, &MeshToResize, &ResizedMesh);
	}
}