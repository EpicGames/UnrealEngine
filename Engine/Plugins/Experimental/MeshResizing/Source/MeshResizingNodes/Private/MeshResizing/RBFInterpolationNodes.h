// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MeshResizing/RBFInterpolation.h"

#include "RBFInterpolationNodes.generated.h"

class UDataflowMesh;
class UMaterialInterface;
class USkeletalMesh;

/**
 * Sample points and generate RBF Interpolation data for a given Source mesh.
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FGenerateRBFResizingWeightsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateRBFResizingWeightsNode, "GenerateRBFResizingWeights", "MeshResizing", "Generate RBF Resizing Weights")

public:

	FGenerateRBFResizingWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** The mesh to resize. This is currently unused but may be used to improve point sampling in the future. */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> MeshToResize;

	/** The source mesh to be sampled.*/
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> SourceMesh;

	/** The calculated interpolation points and RBF weights */
	UPROPERTY(meta = (DataflowOutput))
	FMeshResizingRBFInterpolationData InterpolationData;

	/** The number of interpolation points to be sampled. */
	UPROPERTY(EditAnywhere, Category = "RBF Resizing Weights", meta = (Min = 1, DataflowInput))
	int32 NumInterpolationPoints = 1500;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override; 
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
	//~ End FDataflowNode interface
};

/**
 * Apply the interpolation data calculated by GenerateRBFResizingWeights to resize a mesh.
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FApplyRBFResizingNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FApplyRBFResizingNode, "ApplyRBFResizing", "MeshResizing", "Apply RBF Resizing")

public:
	FApplyRBFResizingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** The mesh being resized */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const UDataflowMesh> MeshToResize;

#if WITH_EDITORONLY_DATA
	// Skeletal mesh target requires access to the MeshDescription, which is Editor-only
	/** Use a skeletal mesh for the target mesh (instead of a dynamic mesh)*/
	UPROPERTY(EditAnywhere, Category = "RBF Resize")
	bool bUseSkeletalMeshTarget = false;
#else
	static constexpr bool bUseSkeletalMeshTarget = false;
#endif

	/** The target mesh that corresponds with the SourceMesh used to generate the InterpolationData. Must have matching vertices with SourceMesh */
	UPROPERTY(meta = (DataflowInput, EditCondition = "bUseSkeletalMeshTarget"))
	TObjectPtr<const USkeletalMesh> TargetSkeletalMesh;

	UPROPERTY(EditAnywhere, Category = "RBF Resize", meta = (DataflowInput, EditCondition = "bUseSkeletalMeshTarget", ClampMin = 0))
	int32 TargetSkeletalMeshLODIndex = 0;

	/** The target mesh that corresponds with the SourceMesh used to generate the InterpolationData. Must have matching vertices with SourceMesh */
	UPROPERTY(meta = (DataflowInput, EditCondition = "!bUseSkeletalMeshTarget"))
	TObjectPtr<const UDataflowMesh> TargetMesh;

	/** The pre-calculated base RBF interpolation data.*/
	UPROPERTY(meta = (DataflowInput))
	FMeshResizingRBFInterpolationData InterpolationData;

	/** The resulting resized mesh */
	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "MeshToResize"))
	TObjectPtr<const UDataflowMesh> ResizedMesh;

	/** Whether or not to interpolate the normals as well as the positions*/
	UPROPERTY(EditAnywhere, Category = "RBF Resize")
	bool bInterpolateNormals = true;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};