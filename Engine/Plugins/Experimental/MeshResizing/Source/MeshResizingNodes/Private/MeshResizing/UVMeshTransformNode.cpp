// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshResizing/UVMeshTransformNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowMesh.h"
#include "UDynamicMesh.h"
#include "Math/TransformCalculus2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVMeshTransformNode)

using UE::Geometry::FDynamicMesh3;

namespace UE::MeshResizing
{
	void RegisterUVMeshTransformNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUVMeshTransformNode);
	}
}

FUVMeshTransformNode::FUVMeshTransformNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&UVChannelIndex);
	RegisterOutputConnection(&Mesh, &Mesh);
	RegisterOutputConnection(&UVChannelIndex, &UVChannelIndex);
}

void FUVMeshTransformNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	using namespace UE::MeshResizing;

	if (Out->IsA(&UVChannelIndex))
	{
		SafeForwardInput(Context, &UVChannelIndex, &UVChannelIndex);
		return;
	}
	else if (Out->IsA(&Mesh))
	{
		if (const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &Mesh))
		{
			if (InMesh->GetDynamicMesh())
			{
				const int32 UVChannel = GetValue(Context, &UVChannelIndex);

				if (InMesh->GetDynamicMeshRef().HasAttributes() && InMesh->GetDynamicMeshRef().Attributes()->NumUVLayers() > UVChannel && UVChannel >= 0)
				{
					const FDynamicMeshUVOverlay* const InMeshUVOverlay = InMesh->GetDynamicMeshRef().Attributes()->GetUVLayer(UVChannel);

					FDynamicMesh3 OutMesh;
					OutMesh.Copy(InMesh->GetDynamicMeshRef());
					FDynamicMeshUVOverlay* const OutMeshUVOverlay = OutMesh.Attributes()->GetUVLayer(UVChannel);

					checkf(InMeshUVOverlay->ElementCount() == OutMeshUVOverlay->ElementCount(), TEXT("Newly created UV overlay doesn't have the same size as input UV overlay"));

					const FTransform2f Transform = FTransform2f{ FScale2f(Scale) }.Concatenate(FTransform2f{ FQuat2f{FMath::DegreesToRadians(Rotation)}, Translation });

					for (int32 ElementIndex : InMeshUVOverlay->ElementIndicesItr())
					{
						const FVector2f NewUV = Transform.TransformPoint(InMeshUVOverlay->GetElement(ElementIndex));
						OutMeshUVOverlay->SetElement(ElementIndex, NewUV);
					}

					TObjectPtr<UDataflowMesh> OutDataflowMesh = NewObject<UDataflowMesh>();
					OutDataflowMesh->SetDynamicMesh(MoveTemp(OutMesh));
					OutDataflowMesh->SetMaterials(InMesh->GetMaterials());
					SetValue(Context, OutDataflowMesh, &Mesh);
					SetValue(Context, UVChannel, &UVChannelIndex);
					return;
				}
				else
				{
					Context.Warning(TEXT("Invalid UVChannelIndex or the input mesh does not have an AttributeSet"), this, Out);
				}
			}
			else
			{
				Context.Warning(TEXT("Input mesh does not have a DynamicMesh object"), this, Out);
			}
		}

		SafeForwardInput(Context, &Mesh, &Mesh);
	}	
}

