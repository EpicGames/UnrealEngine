// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "UVMeshTransformNode.generated.h"

class UDataflowMesh;

USTRUCT(Meta = (MeshResizing, Experimental))
struct FUVMeshTransformNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUVMeshTransformNode, "UVMeshTransformNode", "MeshResizing", "UV Mesh Transform")

public:

	FUVMeshTransformNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDataflowMesh> Mesh;

	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "UVChannelIndex", DataflowUVChannel, ClampMin = 0, ClampMax = 7, UIMin = 0, UIMax = 7))
	int32 UVChannelIndex = 0;

	UPROPERTY(EditAnywhere, Category = "Transform", Meta = (AllowPreserveRatio))
	FVector2f Scale = { 1.f, 1.f };

	/** Rotation angle in degrees */
	UPROPERTY(EditAnywhere, Category = "Transform", Meta = (UIMin = -360, UIMax = 360, ClampMin = -360, ClampMax = 360))
	float Rotation = 0.f;

	UPROPERTY(EditAnywhere, Category = "Transform")
	FVector2f Translation = { 0.f, 0.f };

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};



namespace UE::MeshResizing
{
	void RegisterUVMeshTransformNodes();
}
