// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshResizing/UVResizeControllerNode.h"
#include "Dataflow/DataflowMesh.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Engine/Texture.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVResizeControllerNode)

namespace UE::Chaos::OutfitAsset::Private
{
	static constexpr int32 MaxUVChannelsForResizing = MAX_TEXCOORDS / 2;

	static int32 GetNumUVLayers(const TObjectPtr<const UDataflowMesh> DataflowMesh)
	{
		using namespace UE::Geometry;

		if (const FDynamicMesh3* const DynamicMesh = DataflowMesh->GetDynamicMesh())
		{
			if (const FDynamicMeshAttributeSet* const Attributes = DynamicMesh->Attributes())
			{
				return Attributes->NumUVLayers();
			}
		}
		return 0;
	}

	// Find all UV channel indices parameter names
	// A good candidate must:
	// 1. Have a matching name to a texture parameter name, which only differ by their predetermined suffixes.
	// 2. Have the UVChannel index value be less than the number of UV layers.
	// 3. Be related to a texture that is set to wrap around.
	static TArray<FString> FindUVChannelParameterNames(
		const TObjectPtr<const UMaterialInterface>& Material,
		const FString& TextureSuffix,
		const FString& UVChannelSuffix,
		const int32 NumUVLayers)
	{
		TArray<FString> UVChannelParameterNames;

		if (Material)
		{
			// Iterate through all textures linked to this material
			TArray<FMaterialParameterInfo> TextureParameterInfos;
			TArray<FGuid> TextureParameterIds;
			Material->GetAllTextureParameterInfo(TextureParameterInfos, TextureParameterIds);

			for (const FMaterialParameterInfo& TextureParameterInfo : TextureParameterInfos)
			{
				// Only deals with textures with name ending by TextureSuffix and replace it with UVChannelSuffix to find the matching UV Channel index
				const FString TextureParameterName = TextureParameterInfo.Name.ToString();
				if (TextureParameterName.EndsWith(TextureSuffix))
				{
					// Found a texture with suffix, try to locate the texture object
					UTexture* Texture;
					if (Material->GetTextureParameterValue(TextureParameterInfo, Texture) && Texture)
					{
						if (Texture->GetTextureAddressX() == TA_Wrap &&
							Texture->GetTextureAddressY() == TA_Wrap)
						{
							// Texture can wrap around, so check for a matching channel parameter
							const FString UVChannelParameterName = TextureParameterName.LeftChop(TextureSuffix.Len()) + UVChannelSuffix;
							float Value;
							if (Material->GetScalarParameterValue(*UVChannelParameterName, Value))
							{
								const int32 UVChannelIndex = (int32)Value;
								if (UVChannelIndex < NumUVLayers)
								{
									UVChannelParameterNames.Emplace(UVChannelParameterName);
								}
							}
						}
					}
				}
			}
		}
		return UVChannelParameterNames;
	}
}

FUVResizeControllerNode::FUVResizeControllerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterOutputConnection(&Mesh, &Mesh);
	RegisterOutputConnection(&UVChannelIndices);
	RegisterOutputConnection(&SourceUVChannelIndices);
	RegisterOutputConnection(&bHasUVChannelsToResize);
}

void FUVResizeControllerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::OutfitAsset;

	if (Out->IsA(&Mesh) || Out->IsA(&UVChannelIndices) || Out->IsA(&bHasUVChannelsToResize))
	{
		bool bOutHasUVChannelsToResize = false;

		// Check available layers
		if (const TObjectPtr<UDataflowMesh> InMesh = GetValue(Context, &Mesh))
		{
#if WITH_EDITOR
			if (GIsEditor)  // For UMaterialInstanceConstant::SetScalarParameterValueEditorOnly and UMaterialInstanceConstant::SetParentEditorOnly
			{
				TArray<int32> UVChannelRemaps;
				UVChannelRemaps.Init(INDEX_NONE, MAX_TEXCOORDS);

				const int32 NumUVLayers = Private::GetNumUVLayers(InMesh);
				if (NumUVLayers > 0 && NumUVLayers < Private::MaxUVChannelsForResizing)
				{
					check(InMesh && InMesh->GetDynamicMesh() && InMesh->GetDynamicMesh()->Attributes());  // Already tested in GetNumUVLayers

					// Create a new dynamic mesh
					UE::Geometry::FDynamicMesh3 DynamicMesh;
					DynamicMesh.Copy(*InMesh->GetDynamicMesh());
					DynamicMesh.EnableAttributes();

					const TArray<TObjectPtr<UMaterialInterface>>& InMaterials = InMesh->GetMaterials();
					TArray<TObjectPtr<UMaterialInterface>> OutMaterials;
					OutMaterials.Reserve(InMaterials.Num());

					for (const TObjectPtr<UMaterialInterface>& Material : InMaterials)
					{
						const TArray<FString> UVChannelParameterNames = Private::FindUVChannelParameterNames(Material, TextureSuffix, UVChannelSuffix, NumUVLayers);
						if (UVChannelParameterNames.Num())
						{
							check(Material);  // Already tested in FindUVChannelParameterNames

							// Re-assign the UVChannels in this new material instance
							TArray<float> UVChannelParameterIndices;
							UVChannelParameterIndices.SetNumZeroed(UVChannelParameterNames.Num());

							for (int32 ParameterIndex = 0; ParameterIndex < UVChannelParameterNames.Num(); ++ParameterIndex)
							{
								const FString& UVChannelParameterName = UVChannelParameterNames[ParameterIndex];

								float Value;
								if (ensure(Material->GetScalarParameterValue(*UVChannelParameterName, Value)))
								{
									const int32 SourceUVChannelIndex = (int32)Value;
									if (UVChannelRemaps.IsValidIndex(SourceUVChannelIndex))
									{
										int32& UVChannelIndex = UVChannelRemaps[SourceUVChannelIndex];
										if (UVChannelIndex == INDEX_NONE)
										{
											// Duplicate UV channel
											UVChannelIndex = DynamicMesh.Attributes()->NumUVLayers();
											DynamicMesh.Attributes()->SetNumUVLayers(UVChannelIndex + 1);
											const UE::Geometry::FDynamicMeshUVOverlay* const SourceUVLayer = DynamicMesh.Attributes()->GetUVLayer(SourceUVChannelIndex);
											check(SourceUVLayer);
											DynamicMesh.Attributes()->GetUVLayer(UVChannelIndex)->Copy(*SourceUVLayer);
										}
										UVChannelParameterIndices[ParameterIndex] = (float)UVChannelIndex;
									}
								}
							}

							// Find or create a material instance with the new UV channel indices
							UMaterialInstanceConstant* MaterialInstance;
							const FString MaterialPackagePath = FPackageName::GetLongPackagePath(Material->GetOutermost()->GetName());
							const FString MaterialName = Material->GetName() + TEXT("_UVResized_");
							const FString MaterialPackageName = FPaths::Combine(MaterialPackagePath, MaterialName);
							FString MaterialPackageNameNumbered;

							for (int32 Suffix = 0; ; ++Suffix)
							{
								// Check if a material instance already exists and is identical
								MaterialPackageNameNumbered = MaterialPackageName + FString::FromInt(Suffix);

								if (const UPackage* const Package = FindPackage(nullptr, *MaterialPackageNameNumbered))
								{
									MaterialInstance = Cast<UMaterialInstanceConstant>(Package->FindAssetInPackage());
								}
								else
								{
									MaterialInstance = nullptr;
									break;  // No existing object named with this suffix, a new material instance will need to be created
								}
								if (MaterialInstance && MaterialInstance->Parent == Material)
								{
									bool bIsIdentical = true;
									for (int32 ParameterIndex = 0; ParameterIndex < UVChannelParameterNames.Num(); ++ParameterIndex)
									{
										const FString& UVChannelParameterName = UVChannelParameterNames[ParameterIndex];
										float Value;
										if (!MaterialInstance->GetScalarParameterValue(*UVChannelParameterName, Value) ||
											Value != UVChannelParameterIndices[ParameterIndex])
										{
											bIsIdentical = false;  // Parameter don't match, check with another Suffix
											break;
										}
									}
									if (bIsIdentical)
									{
										break;  // Found a valid existing instance
									}
								}
							}

							if (!MaterialInstance)
							{
								MaterialInstance = Cast<UMaterialInstanceConstant>(Context.AddAsset(MaterialPackageNameNumbered, UMaterialInstanceConstant::StaticClass()));

								if (ensure(MaterialInstance))
								{
									// Setup the new material instance
									MaterialInstance->SetParentEditorOnly(Material);

									for (int32 ParameterIndex = 0; ParameterIndex < UVChannelParameterNames.Num(); ++ParameterIndex)
									{
										const FString& UVChannelParameterName = UVChannelParameterNames[ParameterIndex];
										MaterialInstance->SetScalarParameterValueEditorOnly(*UVChannelParameterName, UVChannelParameterIndices[ParameterIndex]);
									}

									// Finalize the new material asset creation
									MaterialInstance->PreEditChange(nullptr);
									MaterialInstance->PostEditChange();
								}
							}

							OutMaterials.Emplace(MaterialInstance);
							continue;
						}
						OutMaterials.Emplace(Material);
					}

					// Create the output mesh
					TObjectPtr<UDataflowMesh> OutMesh = NewObject<UDataflowMesh>();
					OutMesh->SetDynamicMesh(MoveTemp(DynamicMesh));
					OutMesh->SetMaterials(OutMaterials);
					SetValue(Context, OutMesh, &Mesh);

					TArray<int32> OutUVChannelIndices;
					TArray<int32> OutSourceUVChannelIndices;
					for (int32 Index = 0; Index < UVChannelRemaps.Num(); ++Index)
					{
						const int32 UVChannelRemap = UVChannelRemaps[Index];
						if (UVChannelRemap != INDEX_NONE)
						{
							OutUVChannelIndices.Emplace(UVChannelRemap);
							OutSourceUVChannelIndices.Emplace(Index);
						}
					}
					SetValue(Context, OutUVChannelIndices.Num() > 0, &bHasUVChannelsToResize);  // Set before the MoveTemp!
					SetValue(Context, MoveTemp(OutUVChannelIndices), &UVChannelIndices);
					SetValue(Context, MoveTemp(OutSourceUVChannelIndices), &SourceUVChannelIndices);

					return;
				}
				else
				{
					Context.Warning(
						FString::Printf(TEXT("[%s] isn't a valid mesh with at least %d free UV channels to allow for UV resizing operations."),
							*InMesh->GetName(),
							Private::MaxUVChannelsForResizing),
						this, Out);
				}
			}
#endif  // #if WITH_EDITOR
		}

		TArray<int32> OutUVChannelIndices;
		TArray<int32> OutSourceUVChannelIndices;
		SafeForwardInput(Context, &Mesh, &Mesh);
		SetValue(Context, MoveTemp(OutUVChannelIndices), &UVChannelIndices);
		SetValue(Context, MoveTemp(OutSourceUVChannelIndices), &SourceUVChannelIndices);
		SetValue(Context, false, &bHasUVChannelsToResize);
	}
}
