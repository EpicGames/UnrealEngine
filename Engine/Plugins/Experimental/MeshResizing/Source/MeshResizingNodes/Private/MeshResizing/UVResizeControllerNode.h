// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "UVResizeControllerNode.generated.h"

class UDataflowMesh;
class UMaterialInterface;

/**
 * UV Resizing logic.
 * Returns whether this dynamic mesh is suitable for UV resizing and which UV channels to use.
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FUVResizeControllerNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUVResizeControllerNode, "UVResizeController", "MeshResizing", "UV Resize Controller")

public:
	FUVResizeControllerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** The texture name suffix . */
	UPROPERTY(EditAnywhere, Category = "UVResizeController")
	FString TextureSuffix = TEXT("Texture");

	/** The suffix to replace the texture name with pointing to the UV channel index scalar parameter. */
	UPROPERTY(EditAnywhere, Category = "UVResizeController")
	FString UVChannelSuffix = TEXT("UVIndex");

	/** The input/output Dataflow mesh. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDataflowMesh> Mesh;

	/** The UV channels to resize. */
	UPROPERTY(Meta = (DataflowOutput))
	TArray<int32> UVChannelIndices;

	/** The matching UV channels on the original source mesh. */
	UPROPERTY(Meta = (DataflowOutput))
	TArray<int32> SourceUVChannelIndices;

	/** Whether the input mesh has any UV channels to resize. */
	UPROPERTY(Meta = (DataflowOutput))
	bool bHasUVChannelsToResize = false;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};
