// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshResizing/UVUnwrapNode.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowMesh.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVUnwrapNode)

namespace UE::MeshResizing
{
	void RegisterUVUnwrapNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUVUnwrapNode);
	}

	namespace Private
	{
		float UVIslandArea(const TArray<int32>& TriangleIndices, const UE::Geometry::FDynamicMeshUVOverlay& UVOverlay)
		{
			float TotalArea = 0.0;
			for (const int32 TID : TriangleIndices)
			{
				FVector2f A, B, C;
				UVOverlay.GetTriElements(TID, A, B, C);
				const float TriangleArea = 0.5 * FMath::Abs((B - A)[0] * (C - A)[1] - (B - A)[1] * (C - A)[0]);
				TotalArea += TriangleArea;
			}
			return TotalArea;
		}

		// Returns -1 or 1 depending on the sign of the first non-degenerate triangle's signed area. Triangles with area smaller than UE_SMALL_NUMBER are considered degenerate. 
		// Returns 0 if no non-degenerate triangles are found.
		int32 FirstNonZeroTriangleOrientation(const TArray<int32>& TriangleIndices, const UE::Geometry::FDynamicMeshUVOverlay& UVOverlay)
		{
			for (const int32 TID : TriangleIndices)
			{
				FVector2f A, B, C;
				UVOverlay.GetTriElements(TID, A, B, C);
				const float TriOrientation = (B - A)[0] * (C - A)[1] - (B - A)[1] * (C - A)[0];
				if (FMath::Abs(TriOrientation) > UE_SMALL_NUMBER)
				{
					return TMathUtil<float>::SignAsInt(TriOrientation);
				}
			}
			return 0;
		}

		void FlipUCoordinates(const TArray<int32>& TriangleIndices, UE::Geometry::FDynamicMeshUVOverlay& UVOverlay)
		{
			TSet<int32> ElementIndices;
			for (const int32 TriID : TriangleIndices)
			{
				const UE::Geometry::FIndex3i Elements = UVOverlay.GetTriangle(TriID);
				ElementIndices.Add(Elements[0]);
				ElementIndices.Add(Elements[1]);
				ElementIndices.Add(Elements[2]);
			}

			float MinU = UE_BIG_NUMBER;
			float MaxU = -UE_BIG_NUMBER;
			for (const int32 ElementID : ElementIndices)
			{
				const FVector2f Elem = UVOverlay.GetElement(ElementID);
				MinU = FMath::Min(MinU, Elem[0]);
				MaxU = FMath::Max(MaxU, Elem[0]);
			}

			for (const int32 ElementID : ElementIndices)
			{
				FVector2f UV = UVOverlay.GetElement(ElementID);
				UV[0] = MaxU - UV[0] + MinU;
				UVOverlay.SetElement(ElementID, UV);
			}
		}
	}

}

FUVUnwrapNode::FUVUnwrapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&UVChannelIndex);
	RegisterOutputConnection(&Mesh, &Mesh);
	RegisterOutputConnection(&UVChannelIndex, &UVChannelIndex);
}

// NOTE: Alternatively, we could use UGeometryScriptLibrary_MeshUVFunctions::RecomputeMeshUVs

void FUVUnwrapNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	using namespace UE::MeshResizing;

	if (Out->IsA(&UVChannelIndex))
	{
		SafeForwardInput(Context, &UVChannelIndex, &UVChannelIndex);
		return;
	}
	else if (Out->IsA(&Mesh))
	{
		if (TObjectPtr<UDataflowMesh> InOutMesh = GetValue(Context, &Mesh))
		{
			if (InOutMesh->GetDynamicMesh() && InOutMesh->GetDynamicMesh()->HasAttributes())
			{
				const int32 UVChannel = GetValue(Context, &UVChannelIndex);

				FDynamicMesh3 EditMesh;
				EditMesh.Copy(InOutMesh->GetDynamicMeshRef());

				if (UVChannel >= 0 && UVChannel < EditMesh.Attributes()->NumUVLayers())
				{
					constexpr bool bCreateIfMissing = false;
					FDynamicMeshUVEditor UVEditor(&EditMesh, UVChannel, bCreateIfMissing);

					if (FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(UVChannel))
					{
						FMeshConnectedComponents UVIslands(&EditMesh);
						UVIslands.FindConnectedTriangles([UVOverlay](int32 Triangle0, int32 Triangle1)
						{
							return UVOverlay ? UVOverlay->AreTrianglesConnected(Triangle0, Triangle1) : false;
						});

						for (const FMeshConnectedComponents::FComponent& Island : UVIslands.Components)
						{
							if (!ensure(Island.Indices.Num() > 0))
							{
								continue;
							}

							const float IslandArea = Private::UVIslandArea(Island.Indices, *UVOverlay);

							if (IslandArea < UE_SMALL_NUMBER)
							{
								continue;
							}

							const int32 InitialOrientation = Private::FirstNonZeroTriangleOrientation(Island.Indices, *UVOverlay);

							switch (Method)
							{
							case EUVUnwrapMethod::ExponentialMap:
								UVEditor.SetTriangleUVsFromExpMap(Island.Indices);
								break;

							case EUVUnwrapMethod::ConformalFreeBoundary:
								UVEditor.SetTriangleUVsFromFreeBoundaryConformal(Island.Indices,  /*bUseExistingUVTopology = */ true);
								break;

							case EUVUnwrapMethod::SpectralConformal:
							default:
								UVEditor.SetTriangleUVsFromFreeBoundarySpectralConformal(Island.Indices, /*bUseExistingUVTopology = */ true, /*bPreserveIrregularity = */ true);
								break;
							}

							const int32 NewOrientation = Private::FirstNonZeroTriangleOrientation(Island.Indices, *UVOverlay);
							if (InitialOrientation != NewOrientation)
							{
								Private::FlipUCoordinates(Island.Indices, *UVOverlay);
							}
						}

						UVEditor.QuickPack();

						TObjectPtr<UDataflowMesh> OutMesh = NewObject<UDataflowMesh>();
						OutMesh->SetDynamicMesh(MoveTemp(EditMesh));
						OutMesh->SetMaterials(InOutMesh->GetMaterials());
						SetValue(Context, OutMesh, &Mesh);
						SetValue(Context, UVChannel, &UVChannelIndex);
						return;
					}
					else
					{
						Context.Warning(TEXT("UVOverlay not found at given UVChannelIndex"), this, Out);
					}
				}
				else
				{
					Context.Warning(TEXT("Invalid UVChannelIndex"), this, Out);
				}
			}
			else
			{
				Context.Warning(TEXT("Mesh is missing DynamicMesh object or AttributeSet"), this, Out);
			}
		}
		SafeForwardInput(Context, &Mesh, &Mesh);
	}
}


