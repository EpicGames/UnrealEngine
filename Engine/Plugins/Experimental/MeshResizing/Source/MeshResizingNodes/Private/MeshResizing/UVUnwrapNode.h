// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "UVUnwrapNode.generated.h"

class UDataflowMesh;
class UMaterialInterface;

UENUM()
enum class EUVUnwrapMethod
{
	ExponentialMap = 0,
	ConformalFreeBoundary = 1,
	SpectralConformal = 2
};

USTRUCT(Meta = (MeshResizing, Experimental))
struct FUVUnwrapNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUVUnwrapNode, "UVUnwrapNode", "MeshResizing", "UV Unwrap")

public:

	FUVUnwrapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDataflowMesh> Mesh;

	UPROPERTY(EditAnywhere, Category = "UV Unwrap", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "UVChannelIndex", DataflowUVChannel, ClampMin = 0, ClampMax = 7, UIMin = 0, UIMax = 7))
	int32 UVChannelIndex = 0;

	UPROPERTY(EditAnywhere, Category = "UV Unwrap")
	EUVUnwrapMethod Method = EUVUnwrapMethod::SpectralConformal;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

namespace UE::MeshResizing
{
	void RegisterUVUnwrapNodes();
}
