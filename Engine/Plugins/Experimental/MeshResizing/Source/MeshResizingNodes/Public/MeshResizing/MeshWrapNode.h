// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowPrimitiveNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Operations/WrapMesh.h"

#include "MeshWrapNode.generated.h"

#define UE_API MESHRESIZINGDATAFLOWNODES_API

class UDataflowMesh;
class UMaterialInterface;
class UMaterial;

/**
 * Mesh Wrap Node landmark. Matched landmarks between source topology and target shape meshes are used to guide the Mesh Wrap operation.
 */
USTRUCT()
struct FMeshWrapLandmark
{
	GENERATED_USTRUCT_BODY()

	/** String name. Landmarks will be matched by comparing identifiers.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap")
	FString Identifier;

	/** Vertex of this landmark. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", meta = (ClampMin = "-1"))
	int32 VertexIndex = INDEX_NONE;
};

/**
 * Matched Mesh Wrap Node landmark correspondence.
 */
USTRUCT()
struct FMeshWrapCorrespondence
{
	GENERATED_USTRUCT_BODY()

	/** Matched landmark name.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap")
	FString Identifier;

	/** Vertex on source topology mesh.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", meta = (ClampMin = "-1"))
	int32 SourceVertexIndex = INDEX_NONE;

	/** Vertex on target shape mesh. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", meta = (ClampMin = "-1"))
	int32 TargetVertexIndex = INDEX_NONE;
};

/** Node for defining landmarks used by MeshWrapNode. The Mesh Wrap Landmark Selection Tool allows generating these landmarks via selection. */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FMeshWrapLandmarksNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshWrapLandmarksNode, "MeshWrapLandmarks", "MeshResizing", "Mesh Wrap Landmarks")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceRender", FName("UDataflowMesh"), "Mesh")

public:
	FMeshWrapLandmarksNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return bCanDebugDraw;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
	//~ End FDataflowNode interface

	// Undo/redo handling for associated tool.
	friend class UMeshWrapLandmarkSelectionTool;
	class FLandmarksNodeChange;
	static TUniquePtr<class FToolCommandChange> UE_API MakeSelectedNodeChange(const FMeshWrapLandmarksNode& Node);

	/** The mesh to define landmarks on. */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDataflowMesh> Mesh;

	/** The defined landmarks (identifier, vertex pair). Can be hand-input/edited or generated using the Mesh Wrap Landmark Selection Tool */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", meta = (DataflowOutput))
	TArray<FMeshWrapLandmark> Landmarks;

	/** Point size of displayed landmarks */
	UPROPERTY(EditAnywhere, Category = "Display", meta = (ClampMin = 0.f))
	float PointSize = 5.f;

	/** Display the landmark indices */
	UPROPERTY(EditAnywhere, Category = "Display")
	bool bShowIndex = true;

	/** Display the landmark identifier strings */
	UPROPERTY(EditAnywhere, Category = "Display")
	bool bShowIdentifier = true;

	// This will be set by UMeshWrapLandmarkSelectionTool to false to not double draw.
	bool bCanDebugDraw = true;
};

/** Dataflow node for wrapping one mesh's topology to another mesh's shape. Uses point landmarks defined by the Mesh Wrap Landmarks node to match corresponding points between the two meshes.*/
USTRUCT(Meta = (MeshResizing, Experimental))
struct FMeshWrapNode : public FDataflowPrimitiveNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshWrapNode, "MeshWrap", "MeshResizing", "Mesh Wrap")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceRender", FName("UDataflowMesh"), "WrappedMesh")

public:

	FMeshWrapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** Input mesh with the desired wrapped mesh topology. */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> SourceTopologyMesh;

	/** Input mesh with the desired wrapped mesh shape. */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDataflowMesh> TargetShapeMesh;

	/** Output wrapped mesh. */
	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "SourceTopologyMesh"))
	TObjectPtr<UDataflowMesh> WrappedMesh;

	/** Landmarks defined on SourceTopologyMesh. TargetShapeLandmarks with matching Identifiers will be used to find correspondences that help improve the wrap. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", meta = (DataflowInput))
	TArray<FMeshWrapLandmark> SourceTopologyLandmarks;

	/** Landmarks defined on TargetShapeMesh. SourceTopologyLandmarks with matching Identifiers will be used to find correspondences that help improve the wrap. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", meta = (DataflowInput))
	TArray<FMeshWrapLandmark> TargetShapeLandmarks;

	/** Landmarks matched by Identifier from SourceTopologyLandmarks and TargetShapeLandmarks. */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FMeshWrapCorrespondence> MatchedLandmarks;

	/** Mesh Wrap is calculated with an inner and outer loop. This is the maximum number of outer loops. Each outer loop increases the Projection Stiffness by ProjectionStiffnessMultiplier.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", Meta = (ClampMin ="0"))
	int32 MaxNumOuterIterations = 10;

	/** Mesh Wrap is calculated with an inner and outer loop. This is the number of inner loops run before increasing the Projection Stiffness in the outer loop.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", Meta = (ClampMin = "0"))
	int32 NumInnerIterations = 20;

	/** Mesh Wrap will terminate early if the Projection tolerance is within this threshold.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", Meta =(ClampMin = "0"))
	float ProjectionTolerance = 1e-4f;

	/** Weight of mesh wrap to retain Source Topology mesh features.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", AdvancedDisplay, Meta = (ClampMin = "0"))
	float LaplacianStiffness = 1.;

	/** Initial weight of mesh wrap to match projected Target Shape. Each outer loop will multiply this stiffness by ProjectionStiffnessMultiplier.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", AdvancedDisplay, Meta = (ClampMin = "0"))
	float InitialProjectionStiffness = 0.1f;
	
	/** Each outer loop will multiply InitialProjectionStiffness by this to improve Target Shape match.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", AdvancedDisplay, Meta = (ClampMin = "0"))
	float ProjectionStiffnessMuliplier = 10.;

	/** Weight of mesh wrap to match Landmark correspondences.*/
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap", AdvancedDisplay, Meta = (ClampMin = "0"))
	float CorrespondenceStiffness = 1.;

	/** Display material for Source or Target when none is supplied. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap Display")
	TObjectPtr<UMaterial> DefaultDisplayMaterial;

	/** Display landmarks. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap Display")
	bool bDisplayLandmarks = true;

	/** Draw source mesh. */
	UPROPERTY(EditAnywhere, Category ="Mesh Wrap Display")
	bool bDisplaySource = false;

	/** Offset of source mesh display for side-by-side drawing. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap Display", Meta =(EditCondition = "bDisplaySource"))
	float SourceDisplayOffset = -150.f;

	/** Draw target mesh. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap Display")
	bool bDisplayTarget = false;

	/** Offset of target mesh display for side-by-side drawing. */
	UPROPERTY(EditAnywhere, Category = "Mesh Wrap Display", Meta = (EditCondition = "bDisplayTarget"))
	float TargetDisplayOffset = 150.f;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
	//~ End FDataflowNode interface

	//~ Begin FDataflowPrimitiveNode interface 
	virtual bool HasRenderCollectionPrimitives() const override
	{
		return false;
	}
	virtual void AddPrimitiveComponents(UE::Dataflow::FContext& Context, const TSharedPtr<const FManagedArrayCollection> RenderCollection, TObjectPtr<UObject> NodeOwner,
		TObjectPtr<AActor> RootActor, TArray<TObjectPtr<UPrimitiveComponent>>& PrimitiveComponents) override;
	//~ End FDataflowPrimitiveNode interface

	TArray<FMeshWrapCorrespondence> CalculateMatchedLandmarks(UE::Dataflow::FContext& Context) const;
};

namespace UE::MeshResizing
{
	void RegisterMeshWrapNodes();
}

#undef UE_API